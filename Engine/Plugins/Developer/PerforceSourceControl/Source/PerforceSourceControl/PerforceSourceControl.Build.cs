// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PerforceSourceControl : ModuleRules
{
	public PerforceSourceControl(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.KeepAsIsForNow;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"SourceControl",
				"TypedElementFramework",
			}
		);

		// See SOURCE_CONTROL_WITH_SLATE in SourceControl.Build.cs
		if (Target.bUsesSlate)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"InputCore",
					"Slate",
					"SlateCore"
				}
			);
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Perforce");

		if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Mac)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
