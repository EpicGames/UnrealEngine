// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPerforceSourceControlProvider;

class IPerforceSourceControlWorker
{
public:
	static void RegisterWorkers();
	static TSharedPtr<class IPerforceSourceControlWorker, ESPMode::ThreadSafe> CreateWorker(const FName& OperationName, FPerforceSourceControlProvider& SCCProvider);
public:

	IPerforceSourceControlWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: SourceControlProvider(InSourceControlProvider)
	{

	}

	virtual ~IPerforceSourceControlWorker() = default;

	/**
	 * Name describing the work that this worker does. Used for factory method hookup.
	 */
	virtual FName GetName() const = 0;

	/**
	 * Function that actually does the work. Can be executed on another thread.
	 */
	virtual bool Execute( class FPerforceSourceControlCommand& InCommand ) = 0;

	/**
	 * Updates the state of any items after completion (if necessary). This is always executed on the main thread.
	 * @returns true if states were updated
	 */
	virtual bool UpdateStates() const = 0;

	FPerforceSourceControlProvider& GetSCCProvider() const
	{
		return SourceControlProvider;
	}

private:
	 
	FPerforceSourceControlProvider& SourceControlProvider;
};

typedef TSharedRef<IPerforceSourceControlWorker, ESPMode::ThreadSafe> FPerforceSourceControlWorkerRef;
