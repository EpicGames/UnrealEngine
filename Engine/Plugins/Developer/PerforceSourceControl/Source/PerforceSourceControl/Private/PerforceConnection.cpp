// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceConnection.h"

#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "ISourceControlModule.h"
#include "Logging/MessageLog.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "PerforceMessageLog.h"
#include "PerforceSourceControlProvider.h"
#include "PerforceSourceControlSettings.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SourceControlOperations.h"

#define LOCTEXT_NAMESPACE "PerforceConnection"

#define FROM_TCHAR(InText, bIsUnicodeServer) (bIsUnicodeServer ? TCHAR_TO_UTF8(InText) : TCHAR_TO_ANSI(InText))
#define TO_TCHAR(InText, bIsUnicodeServer) (bIsUnicodeServer ? UTF8_TO_TCHAR(InText) : ANSI_TO_TCHAR(InText))

const FString FP4Record::EmptyStr;

namespace
{

bool bDebugBreakConnection = false;
static FAutoConsoleVariableRef CVarDebugBreakConnection(
	TEXT("SourceControl.Perforce.Debug.BreakConnection"),
	bDebugBreakConnection,
	TEXT("Fails any attempt to run a Perforce command."),
	ECVF_Default);

enum class EP4ClientUserFlags
{
	None = 0,
	/** The server uses unicode */
	UnicodeServer = 1 << 0,
	/** Binary data returned by commands should be collected in the DataBuffer member */
	CollectData = 1 << 1,
};

ENUM_CLASS_FLAGS(EP4ClientUserFlags);

/**
 * A utility class to make it easier to gather a depot file from perforce when running
 * p4 print.
 */
class FP4DepotFile
{
public:
	FP4DepotFile() = default;
	~FP4DepotFile() = default;

	/** 
	 * Start gathering the file in the given record. If the record is missing data
	 * then the gather will not begin. The calling code should check for this and
	 * raise errors or warnings accordingly.
	 * This class does not actually do any perforce work itself, and relies on a
	 * ClientUser to actually provide the data as it is downloaded.
	 */
	void Initialize(const FP4Record& Record)
	{
		const FString& SizeAsString = Record(TEXT("fileSize"));
		DepotFilePath = Record(TEXT("depotFile"));

		if (!SizeAsString.IsEmpty() && !DepotFilePath.IsEmpty())
		{
			int64 FileSize = FCString::Atoi64(*SizeAsString);

			Data = FUniqueBuffer::Alloc(FileSize);
			Offset = 0;
		}
	}

	/** Returns true if the FP4DepotFile was set up correctly and can gather a file. */
	bool IsValid() const
	{
		return !Data.IsNull();
	}

	/** Returns true if all of the files data has been acquired. */
	bool IsFileComplete() const
	{
		return Offset == (int64)Data.GetSize();
	}

	/** Returns the number of bytes in the file that have not yet been acquired. */
	int64 GetRemainingBytes() const
	{
		return (int64)Data.GetSize() - Offset;
	}

	/** Returns the depot path of the file we are gathering */
	const FString& GetDepotPath() const
	{
		return DepotFilePath;
	}

	/** 
	 * Returns the currently acquired file data and then invalidates the FP4DepotFile.
	 * It is up to the caller to ensure that the entire file has been acquired or to
	 * decide if a partially acquired file is okay.
	 */
	FSharedBuffer Release()
	{
		Offset = INDEX_NONE;

		DepotFilePath.Reset();

		return Data.MoveToShared();
	}

	/* Used to reset the FP4DepotFile if an error is encountered */
	void Reset()
	{
		Offset = INDEX_NONE;

		DepotFilePath.Reset();

		Data.Reset();
	}

	/** 
	 * Called when new data for the file has been downloaded and we can add it to the
	 * data that we have already required.
	 * 
	 * @param DataPtr		The pointer to the downloaded data
	 * @param DataLength	The size of the downloaded data in bytes
	 * @return				True if the FP4DepotFile is valid and there was enough space
	 *						for the downloaded data. False if the FP4DepotFile is invalid
	 *						or if there is not enough space.		
	 */
	bool OnDataDownloaded(const char* DataPtr, int DataLength)
	{
		if (DataLength <= GetRemainingBytes())
		{
			FMemory::Memcpy((char*)Data.GetData() + Offset, DataPtr, DataLength);
			Offset += DataLength;

			return true;
		}
		else
		{
			return false;
		}
	}

private:

	/** The path of the file in the perforce depot */
	FString DepotFilePath;

	/** The buffer containing the file data, allocated up front. */
	FUniqueBuffer Data;
	/** Tracks where the next set on downloaded data should be placed in the buffer. */
	int64 Offset = INDEX_NONE;
};

} //namespace

/** Custom ClientUser class for handling results and errors from Perforce commands */
class FP4ClientUser : public ClientUser
{
public:

	FP4ClientUser(FP4RecordSet& InRecords, EP4ClientUserFlags InFlags, FSourceControlResultInfo& OutResultInfo)
		: ClientUser()
		, Flags(InFlags)
		, Records(InRecords)
		, ResultInfo(OutResultInfo)
	{

	}

	/**  Called by P4API when the results from running a command are ready. */
	virtual void OutputStat(StrDict* VarList) override
	{
		FP4Record Record;
		StrRef Var, Value;

		// Iterate over each variable and add to records
		for (int32 Index = 0; VarList->GetVar(Index, Var, Value); Index++)
		{
			Record.Add(TO_TCHAR(Var.Text(), IsUnicodeServer()), TO_TCHAR(Value.Text(), IsUnicodeServer()));
		}

		if (IsCollectingData())
		{
			if (File.IsValid())
			{
				FText Message = FText::Format(LOCTEXT("P4Client_GatheringUnfinished", "Started gathering depot file '{0}' before the previous file finished!"),
					FText::FromString(File.GetDepotPath()));

				ResultInfo.ErrorMessages.Add(MoveTemp(Message));
			}

			File.Initialize(Record);
		}

		Records.Add(Record);
	}

	/** Called by P4API when it output a chunk of text data from a file (commonly via P4 Print) */
	virtual void OutputText(const char* DataPtr, int DataLength) override
	{
		if (IsCollectingData())
		{
			if (File.OnDataDownloaded(DataPtr, DataLength))
			{
				if (File.IsFileComplete())
				{
					Files.Add(File.Release());
				}
			}
			else
			{
				FText Message = FText::Format(	
					LOCTEXT("P4Client_TextCollectionFailed", "Collecting text data requires {0} bytes but the buffer only has {1} bytes remaining: {2}"),
					DataLength,
					File.GetRemainingBytes(),
					FText::FromString(File.GetDepotPath()));

				ResultInfo.ErrorMessages.Add(MoveTemp(Message));

				File.Reset();
			}
		}
		else
		{
			ClientUser::OutputText(DataPtr, DataLength);
		}
	}

	/**  Called by P4API when it output a chunk of binary data from a file (commonly via P4 Print) */
	void OutputBinary(const char* DataPtr, int DataLength)
	{
		if (IsCollectingData())
		{
			// For binary files we get a zero size call once the file is completed so we wait for that
			// rather than checking FP4DepotFile::isFileComplete after every transfer
			if (DataLength == 0)
			{
				if (File.IsFileComplete())
				{
					Files.Add(File.Release());
				}
				else
				{
					FText Message = FText::Format(
						LOCTEXT("P4Client_IncompleteFIle", "Collecting binary data completed but missing {0} bytes: {1}"),
						File.GetRemainingBytes(),
						FText::FromString(File.GetDepotPath()));

					ResultInfo.ErrorMessages.Add(MoveTemp(Message));

					File.Reset();
				}
			}
			else if (!File.OnDataDownloaded(DataPtr, DataLength))
			{
				FText Message = FText::Format(
					LOCTEXT("P4Client_BinaryCollectionFailed", "Collecting binary data requires {0} bytes but the buffer only has {1} bytes remaining: {2}"),
					DataLength,
					File.GetRemainingBytes(),
					FText::FromString(File.GetDepotPath()));

				ResultInfo.ErrorMessages.Add(MoveTemp(Message));

				File.Reset();
			}
		}
		else
		{
			ClientUser::OutputText(DataPtr, DataLength);
		}
	}

	virtual void Message(Error* err) override
	{
		StrBuf Buffer;
		err->Fmt(Buffer, EF_PLAIN);

		FString Message(TO_TCHAR(Buffer.Text(), IsUnicodeServer()));

		// Previously we used ::HandleError which would have \n at the end of each line.
		// For now we should add that to maintain compatibility with existing code.
		if (!Message.EndsWith(TEXT("\n")))
		{
			Message.Append(TEXT("\n"));
		}

		if (err->GetSeverity() <= ErrorSeverity::E_INFO)
		{
			ResultInfo.InfoMessages.Add(FText::FromString(MoveTemp(Message)));
		}
		else
		{
			ResultInfo.ErrorMessages.Add(FText::FromString(MoveTemp(Message)));
		}
	}

	virtual void OutputInfo(char Indent, const char* InInfo) override
	{
		// We don't expect this to ever be called (info messages should come
		// via ClientUser::Message) but implemented just to be safe.

		ResultInfo.InfoMessages.Add(FText::FromString(FString(TO_TCHAR(InInfo, IsUnicodeServer()))));
	}

	virtual void OutputError(const char* errBuf) override
	{
		// In general we expect errors to be passed to use via ClientUser::Message but some
		// errors raised by the p4 cpp api can call ::HandleError or ::OutputError directly.
		// Since the default implementation of ::HandleError calls ::OutputError we only need
		// to implement this method to make sure we capture all of the errors being passed in
		// this way.

		ResultInfo.ErrorMessages.Add(FText::FromString(FString(TO_TCHAR(errBuf, IsUnicodeServer()))));
	}

	inline bool IsUnicodeServer() const
	{
		return EnumHasAnyFlags(Flags, EP4ClientUserFlags::UnicodeServer);
	}

	inline bool IsCollectingData() const
	{
		return EnumHasAnyFlags(Flags, EP4ClientUserFlags::CollectData);
	}

	/** Returns DataBuffer as a FSharedBuffer, note that once called DataBuffer will be empty */
	inline TArray<FSharedBuffer> ReleaseData()
	{
		return MoveTemp(Files);
	}

	EP4ClientUserFlags Flags;
	FP4RecordSet& Records;
	FSourceControlResultInfo& ResultInfo;

private:
	TArray<FSharedBuffer> Files;

	FP4DepotFile File;
};

/** A class used instead of FP4ClientUser for handling changelist create command */
class FP4CreateChangelistClientUser : public FP4ClientUser
{
public:
	FP4CreateChangelistClientUser(FP4RecordSet& InRecords, EP4ClientUserFlags InFlags, FSourceControlResultInfo& OutResultInfo, const FText& InDescription, const TArray<FString>& InFiles, ClientApi &InP4Client)
		:	FP4ClientUser(InRecords, InFlags, OutResultInfo)
		,	Description(InDescription)
		,	ChangelistNumber(0)
		,	Files(InFiles)
		,	P4Client(InP4Client)
	{}

	/** Called by P4API on "change" command. FileSys is a file pointer to the input contents that p4 change will use. This comes with a default template for the server (see p4 change)*/
	virtual void Edit(FileSys* f1, Error* e) override
	{
		StrBuf ReadBuffer;
		f1->ReadFile(&ReadBuffer, e);

		FString TemplateContents = TO_TCHAR(ReadBuffer.Value(), IsUnicodeServer());
		TArray<FString> Lines;
		TemplateContents.ParseIntoArrayLines(Lines, false);

		TStringBuilder<2048> FinalContents;
		bool bFileSectionPresent= false;

		for (size_t i = 0; i < Lines.Num(); ++i)
		{
			const FString& Line = Lines[i];

			// Ignore comments in the template and keep everything else
			if (!Line.StartsWith(TEXT("#")))
			{
				// When we find the Description, replace it with our own.
				if (Line.StartsWith(TEXT("Description:")))
				{
					FinalContents << Line << TEXT("\n");

					TArray<FString> DescLines;
					Description.ToString().ParseIntoArray(DescLines, TEXT("\n"), false);
					for (const FString& DescLine : DescLines)
					{
						FinalContents << TEXT("\t") << DescLine << TEXT("\n");
					}

					// Skip the next line after "Description:" which will be "<enter description here>"
					++i;
				}
				// When we find the File section, remove it completely to create an empty CL
				// Or replace them with our own files if we were provided any
				else if (Line.StartsWith(TEXT("Files:")))
				{
					bFileSectionPresent = true;

					if (Files.Num() != 0)
					{
						FinalContents << Line << TEXT("\n");

						for (const FString& FileName : Files)
						{
							FinalContents << TEXT("\t") << FileName << TEXT("\n");
						}

						FinalContents << TEXT("\n");
					}

					// Skip the default files up to empty line
					// Sections in the p4 change command are separated by empty lines
					do
					{
						++i;
					}
					while (i < Lines.Num() && !Lines[i].IsEmpty());
				}
				else
				{
					FinalContents << Line << TEXT("\n");
				}
			}
		}

		if (!bFileSectionPresent && Files.Num() != 0)
		{
			FinalContents << TEXT("Files:\n");

			for (const FString& FileName : Files)
			{
				FinalContents << TEXT("\t") << FileName << TEXT("\n");
			}

			FinalContents << TEXT("\n");
		}

		StrBuf WriteBuffer(FROM_TCHAR(*FinalContents, IsUnicodeServer()));
		f1->WriteFile(&WriteBuffer, e);
	}

	/** Called by P4API when the changelist is created. */
	virtual void Message(Error* err) override
	{
		if (err->GetSeverity() <= ErrorSeverity::E_INFO)
		{
			StrBuf Buffer;
			err->Fmt(Buffer, EF_PLAIN);

			FString Message(TO_TCHAR(Buffer.Text(), IsUnicodeServer()));

			const int32 ChangeTextLen = FCString::Strlen(TEXT("Change "));
			if (Message.StartsWith(TEXT("Change ")))
			{
				ChangelistNumber = FCString::Atoi(*Message + ChangeTextLen);
			}
		}

		// Pass the message on as we will still want to record it
		FP4ClientUser::Message(err);
	}

	FText Description;
	int32 ChangelistNumber;
	TArray<FString> Files;
	ClientApi& P4Client;
};

/** A class used instead of FP4ClientUser for handling changelist edit command */
class FP4EditChangelistClientUser : public FP4ClientUser
{
public:
	FP4EditChangelistClientUser(FP4RecordSet& OutRecords, EP4ClientUserFlags InFlags, FSourceControlResultInfo& OutResultInfo, const FText& InDescription, int32 InChangelistNumber, const FP4RecordSet& InRecords, ClientApi& InP4Client)
		: FP4ClientUser(OutRecords, InFlags, OutResultInfo)
		, Description(InDescription)
		, ChangelistNumber(InChangelistNumber)
		, P4Client(InP4Client)
		, PreviousRecords(InRecords)
	{
	}

	/** Called by P4API when the changelist is updated. */
	virtual void Message(Error* err) override
	{
		if (err->GetSeverity() <= ErrorSeverity::E_INFO)
		{
			StrBuf Buffer;
			err->Fmt(Buffer, EF_PLAIN);

			FString Message(TO_TCHAR(Buffer.Text(), IsUnicodeServer()));

			const int32 ChangeTextLen = FCString::Strlen(TEXT("Change "));
			if (Message.StartsWith(TEXT("Change ")))
			{
				ChangelistNumber = FCString::Atoi(*Message + ChangeTextLen);
			}
		}

		// Pass the message on as we will still want to record it
		FP4ClientUser::Message(err);
	}

	/** Called by P4API on "change -i" command. OutBuffer is filled with changelist specification text. */
	virtual void InputData(StrBuf* OutBuffer, Error* OutError) override
	{
		FString OutputDesc;
		const FP4Record& Record = PreviousRecords[0];

		for (const auto& Field : Record)
		{
			// Skip some fields that aren't required (as evidenced by the CreateChangelist before
			if (Field.Key == TEXT("Date") ||
				Field.Key == TEXT("Type") ||
				Field.Key == TEXT("specFormatted") ||
				Field.Key == TEXT("func"))
			{
				continue;
			}
			else if (Field.Key == TEXT("extraTag0"))
			{
				// Changelists with shelved files will have additional tags which will be rejected in the edit
				break;
			}

			if (Field.Key == TEXT("Description"))
			{
				// Description case: we will replace the current description by the new one provided
				OutputDesc += TEXT("Description:\n");
				{
					TArray<FString> DescLines;
					Description.ToString().ParseIntoArray(DescLines, TEXT("\n"), false);
					for (const FString& DescLine : DescLines)
					{
						OutputDesc += TEXT("\t");
						OutputDesc += DescLine;
						OutputDesc += TEXT("\n");
					}
				}
				OutputDesc += TEXT("\n");
			}
			else if (Field.Key.Contains(TEXT("Files")))
			{
				if (Field.Key == TEXT("Files0"))
				{
					OutputDesc += TEXT("Files:\n");
				}

				OutputDesc += TEXT("\t");
				OutputDesc += Field.Value;
				OutputDesc += TEXT("\n\n");
			}
			else
			{
				// General case: just put back what is already present in the record
				OutputDesc += Field.Key;
				OutputDesc += TEXT(":\t");
				OutputDesc += Field.Value;
				OutputDesc += TEXT("\n\n");
			}
		}

		OutBuffer->Append(FROM_TCHAR(*OutputDesc, IsUnicodeServer()));
	}

	FText Description;
	int32 ChangelistNumber;
	ClientApi& P4Client;
	FP4RecordSet PreviousRecords;
};

/** An extended version of FP4ClientUser that allows the standard input stream to be overriden */
class FP4CommandWithStdInputClientUser : public FP4ClientUser
{
public:
	FP4CommandWithStdInputClientUser(FStringView InStdInput, FP4RecordSet& InRecords, EP4ClientUserFlags InFlags, FSourceControlResultInfo& OutResultInfo, ClientApi& InP4Client)
		: FP4ClientUser(InRecords, InFlags, OutResultInfo)
		, P4Client(InP4Client)
		, StdInput(InStdInput)
	{
	}


	virtual void InputData(StrBuf* OutBuffer, Error* OutError) override
	{	
		OutBuffer->Append(FROM_TCHAR(*StdInput, IsUnicodeServer()));
	}

protected:
	ClientApi& P4Client;
	FString StdInput;
};

/** Custom ClientUser class for handling login commands */
class FP4LoginClientUser : public FP4ClientUser
{
public:
	FP4LoginClientUser(const FString& InPassword, FP4RecordSet& InRecords, EP4ClientUserFlags InFlags, FSourceControlResultInfo& OutResultInfo)
		:	FP4ClientUser(InRecords, InFlags, OutResultInfo)
		,	Password(InPassword)
	{
	}

	/** Called when we are prompted for a password */
	virtual void Prompt( const StrPtr& InMessage, StrBuf& OutPrompt, int NoEcho, Error* InError ) override
	{
		OutPrompt.Set(FROM_TCHAR(*Password, IsUnicodeServer()));
	}

	/** Password to use when logging in */
	FString Password;
};


class FP4KeepAlive : public KeepAlive
{
public:
	FP4KeepAlive(FOnIsCancelled InIsCancelled)
		: IsCancelled(InIsCancelled)
	{
	}

	/** Called when the perforce connection wants to know if it should stay connected */
	virtual int IsAlive() override
	{
		if(IsCancelled.IsBound() && IsCancelled.Execute())
		{
			return 0;
		}

		return 1;
	}

	FOnIsCancelled IsCancelled;
};

static bool TestLoginConnection(ClientApi& P4Client, bool bIsUnicodeServer, TArray<FText>& OutErrorMessages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::TestLoginConnection);

	OutErrorMessages.Reset();

	FSourceControlResultInfo ResultInfo;
	FP4RecordSet Records;
	EP4ClientUserFlags Flags = bIsUnicodeServer ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;
	FP4ClientUser User(Records, Flags, ResultInfo);

	const char* ArgV[] = { "-s" };

	P4Client.SetArgv(1, const_cast<char* const*>(ArgV));
	P4Client.Run("login", &User);

	OutErrorMessages = MoveTemp(ResultInfo.ErrorMessages);

	return OutErrorMessages.IsEmpty();
}

/**
 * Runs "client" command to test if the connection is actually OK. ClientApi::Init() only checks
 * if it can connect to server, doesn't verify user name nor workspace name.
 */
static bool TestClientConnection(ClientApi& P4Client, const FString& ClientSpecName, bool bIsUnicodeServer, TArray<FText>& OutErrorMessages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::TestClientConnection);
	
	OutErrorMessages.Reset();

	FSourceControlResultInfo ResultInfo;
	FP4RecordSet Records;
	EP4ClientUserFlags Flags = bIsUnicodeServer ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;
	FP4ClientUser User(Records, Flags, ResultInfo);

	UTF8CHAR* ClientSpecUTF8Name = nullptr;
	if(bIsUnicodeServer)
	{
		FTCHARToUTF8 UTF8String(*ClientSpecName);
		ClientSpecUTF8Name = new UTF8CHAR[UTF8String.Length() + 1];
		FMemory::Memcpy(ClientSpecUTF8Name, UTF8String.Get(), UTF8String.Length() + 1);
	}
	else
	{		
		ClientSpecUTF8Name = new UTF8CHAR[ClientSpecName.Len() + 1];
		FMemory::Memcpy(ClientSpecUTF8Name, TCHAR_TO_ANSI(*ClientSpecName), ClientSpecName.Len() + 1);
	}

	const char *ArgV[] = { "-o", (char*)ClientSpecUTF8Name };

	P4Client.SetArgv(2, const_cast<char*const*>(ArgV));
	P4Client.Run("client", &User);

	// clean up args
	delete [] ClientSpecUTF8Name;

	OutErrorMessages = MoveTemp(ResultInfo.ErrorMessages);

	// If there are error messages, user name is most likely invalid. Otherwise, make sure workspace actually
	// exists on server by checking if we have it's update date.
	bool bConnectionOK = OutErrorMessages.Num() == 0 && Records.Num() > 0 && Records[0].Contains(TEXT("Update"));
	if (!bConnectionOK && OutErrorMessages.Num() == 0)
	{
		OutErrorMessages.Add(LOCTEXT("InvalidWorkspace", "Invalid Workspace"));
	}

	return bConnectionOK;
}

static bool CheckUnicodeStatus(ClientApi& P4Client, bool& bIsUnicodeServer, TArray<FText>& OutErrorMessages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::CheckUnicodeStatus);

	OutErrorMessages.Reset();

	FSourceControlResultInfo ResultInfo;
	FP4RecordSet Records;
	FP4ClientUser User(Records, EP4ClientUserFlags::None, ResultInfo);

	P4Client.Run("info", &User);

	OutErrorMessages = MoveTemp(ResultInfo.ErrorMessages);

	if(Records.Num() > 0)
	{
		bIsUnicodeServer = Records[0].Find(TEXT("unicode")) != nullptr;
	}
	else
	{
		bIsUnicodeServer = false;
	}

	return OutErrorMessages.Num() == 0;
}

static void FinalizeErrors(const FPerforceSourceControlProvider& SCCProvider, const FPerforceConnectionInfo& Settings, ISourceControlProvider::FInitResult::FConnectionErrors& OutConnectionErrors)
{
	if (OutConnectionErrors.AdditionalErrors.IsEmpty())
	{
		OutConnectionErrors.AdditionalErrors.Add(LOCTEXT("P4_UnknownError", "Unknown error"));
	}
	else
	{
		// Some perforce errors end with newline characters which can cause odd formatting
		// when we display/log the errors, so take this opportunity to trim them if needed.
		for (FText& MsgText : OutConnectionErrors.AdditionalErrors)
		{
			const FString Msg = MsgText.ToString();
			if (!Msg.IsEmpty() && FChar::IsWhitespace(Msg[Msg.Len() - 1]))
			{
				MsgText = FText::FromString(Msg.TrimEnd());
			}	
		}
	}

	// Not really an error message but when we do have errors it is useful to print out a summary of the current settings.

	TArray<FText> Components;
	Components.Add(FText::Format(LOCTEXT("P4_OwningSystem", "OwningSystem={0}"), FText::FromString(SCCProvider.GetOwnerName())));

	if (!Settings.Port.IsEmpty())
	{
		Components.Add(FText::Format(LOCTEXT("P4_Port", "Port={0}"), FText::FromString(Settings.Port)));
	}

	if (!Settings.UserName.IsEmpty())
	{
		Components.Add(FText::Format(LOCTEXT("P4_User", "User={0}"), FText::FromString(Settings.UserName)));
	}

	if (!Settings.Workspace.IsEmpty())
	{
		Components.Add(FText::Format(LOCTEXT("P4_ClientSpec", "ClientSpec={0}"), FText::FromString(Settings.Workspace)));
	}

	OutConnectionErrors.AdditionalErrors.Add(FText::Join(LOCTEXT("P4_Delim", ", "), Components));
}

FPerforceConnection::FPerforceConnection(const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& InSCCProvider, FSourceControlResultInfo* OutResultInfo)
	: bEstablishedConnection(false)
	, bIsUnicode(false)
	, LatestCommunicateTime(0.0)
	, SCCProvider(InSCCProvider)
{
	EstablishConnection(InConnectionInfo, OutResultInfo);
}

FPerforceConnection::~FPerforceConnection()
{
	Disconnect();
}

bool FPerforceConnection::AutoDetectWorkspace(const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& SCCProvider, FString& OutWorkspaceName, TArray<FText>& OutErrorMessages)
{
	//before even trying to summon the window, try to "smart" connect with the default server/username

	OutErrorMessages.Reset();

	FPerforceConnection Connection(InConnectionInfo, SCCProvider, nullptr);
	TArray<FString> ClientSpecList;
	FSourceControlResultInfo ResultInfo;

	Connection.GetProjectWorkspaces(InConnectionInfo, FOnIsCancelled(), ClientSpecList, ResultInfo);

	OutErrorMessages = MoveTemp(ResultInfo.ErrorMessages);

	if (!OutErrorMessages.IsEmpty())
	{
		return false;
	}

	if (ClientSpecList.Num() == 1)
	{
		OutWorkspaceName = ClientSpecList[0];

		FTSMessageLog SourceControlLog("SourceControl");

		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("WorkspaceName"), FText::FromString(OutWorkspaceName) );

		SourceControlLog.Info(FText::Format(LOCTEXT("ClientSpecAutoDetect", "Auto-detected Perforce client spec: '{WorkspaceName}'"), Arguments));
		
		return true;
	}
	else if (ClientSpecList.Num() > 0)
	{
		FTSMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("AmbiguousClientSpecLine1", "Revision Control unable to auto-login due to ambiguous client specs"));
		SourceControlLog.Warning(LOCTEXT("AmbiguousClientSpecLine2", "  Please select a client spec in the Perforce settings dialog"));
		SourceControlLog.Warning(LOCTEXT("AmbiguousClientSpecLine3", "  If you are unable to work with revision control, consider checking out the files by hand temporarily"));

		// List out the clientspecs that were found to be ambiguous
		SourceControlLog.Info(LOCTEXT("AmbiguousClientSpecListTitle", "Ambiguous client specs..."));
		for (int32 Index = 0; Index < ClientSpecList.Num(); Index++)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add( TEXT("ClientSpecName"), FText::FromString(ClientSpecList[Index]) );
			SourceControlLog.Info(FText::Format(LOCTEXT("AmbiguousClientSpecListItem", "...{ClientSpecName}"), Arguments));
		}

		return false;
	}
	else
	{
		// No clients
		FTSMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("NoClientSpec", "Revision Control unable to auto-login as no client specs were found"));
		
		return false;
	}
}

bool FPerforceConnection::Login(const FPerforceConnectionInfo& InConnectionInfo)
{
	FSourceControlResultInfo ResultInfo;

	FP4RecordSet Records;
	FP4LoginClientUser User(InConnectionInfo.Password, Records, EP4ClientUserFlags::None, ResultInfo);

	const char *ArgV[] = { "-a" };
	P4Client.SetArgv(1, const_cast<char*const*>(ArgV));
	P4Client.Run("login", &User);

	if (ResultInfo.HasErrors())
	{
		UE_LOGF(LogSourceControl, Error, "Login failed");
		for (const FText& ErrorMessage : ResultInfo.ErrorMessages)
		{
			UE_LOGF(LogSourceControl, Error, "%ls", *ErrorMessage.ToString());
		}
	}

	return !ResultInfo.HasErrors();
}

bool EnsureValidConnectionInternal(const FPerforceConnectionInfo& InSettings, FPerforceSourceControlProvider& SCCProvider, EConnectionOptions Options,
	FPerforceConnectionInfo& OutSettings, ISourceControlProvider::FInitResult::FConnectionErrors& OutConnectionErrors)
{
	const bool bRequireWorkspace = !EnumHasAllFlags(Options, EConnectionOptions::WorkspaceOptional);

	ClientApi TestP4;
	TestP4.SetProg("UE");
	TestP4.SetProtocol("tag", "");
	TestP4.SetProtocol("enableStreams", "");

	if (!InSettings.Port.IsEmpty())
	{
		TestP4.SetPort(TCHAR_TO_ANSI(*InSettings.Port));
	}

	if (!InSettings.HostOverride.IsEmpty())
	{
		TestP4.SetHost(TCHAR_TO_ANSI(*InSettings.HostOverride));
	}

	Error P4Error;
	TestP4.Init(&P4Error);

	const bool bConnectionResult = !P4Error.Test();

	// Assume UTF8 for the Port and potential errors as encoding of the port is not affected by the server settings
	OutSettings.Port = UTF8_TO_TCHAR(TestP4.GetPort().Text()); // Record the PORT that we attempted to connect to

	if (!bConnectionResult)
	{
		OutConnectionErrors.ErrorMessage = LOCTEXT("P4_FailedToConnect", "P4ERROR: Failed to connect to revision control server.");

		StrBuf ErrorMessage;
		P4Error.Fmt(&ErrorMessage);

		OutConnectionErrors.AdditionalErrors.Add(FText::FromString(UTF8_TO_TCHAR(ErrorMessage.Text())));
		return false;
	}

	bool bIsUnicodeServer = false;
	if (!CheckUnicodeStatus(TestP4, bIsUnicodeServer, OutConnectionErrors.AdditionalErrors))
	{
		OutConnectionErrors.ErrorMessage = LOCTEXT("P4ErrorConnection_CouldNotDetermineUnicodeStatus", "P4ERROR: Could not determine server unicode status.");
		return false;
	}

	if (bIsUnicodeServer)
	{
		// set translation mode. From here onwards we need to use UTF8 when using text args
		TestP4.SetTrans(CharSetApi::UTF_8);
	}

	// now we have determined unicode status, we can set the values that can be specified in non-ansi characters
	TestP4.SetCwd(FROM_TCHAR(*FPaths::RootDir(), bIsUnicodeServer));
	TestP4.SetUser(FROM_TCHAR(*InSettings.UserName, bIsUnicodeServer));
	TestP4.SetClient(FROM_TCHAR(*InSettings.Workspace, bIsUnicodeServer));
	TestP4.SetPassword(FROM_TCHAR(*InSettings.Ticket, bIsUnicodeServer));

	const bool LogInResult = TestLoginConnection(TestP4, bIsUnicodeServer, OutConnectionErrors.AdditionalErrors);

	OutSettings.UserName = TO_TCHAR(TestP4.GetUser().Text(), bIsUnicodeServer);

	if (!LogInResult)
	{
		OutConnectionErrors.ErrorMessage = LOCTEXT("P4_InvalidToken", "Unable to log into revision control server");
		return false;
	}

	// Try to auto detect the client if none were specified and we require one
	if (bRequireWorkspace)
	{
		FString ClientSpecName = InSettings.Workspace;
		if (ClientSpecName.IsEmpty())
		{
			FPerforceConnectionInfo AutoCredentials = InSettings;
			AutoCredentials.Port = OutSettings.Port;
			AutoCredentials.UserName = OutSettings.UserName;

			// AutoDetectWorkspace takes care of the error reporting
			if (!FPerforceConnection::AutoDetectWorkspace(AutoCredentials, SCCProvider, ClientSpecName, OutConnectionErrors.AdditionalErrors))
			{
				return false;
			}

			TestP4.SetClient(FROM_TCHAR(*ClientSpecName, bIsUnicodeServer));
		}

		// Test that we found a valid client if we require one
		
		const bool bValidClientSpec = TestClientConnection(TestP4, ClientSpecName, bIsUnicodeServer, OutConnectionErrors.AdditionalErrors);

		OutSettings.Workspace = TO_TCHAR(TestP4.GetClient().Text(), bIsUnicodeServer);
		if (!bValidClientSpec)
		{
			OutConnectionErrors.ErrorMessage = LOCTEXT("P4ErrorConnection_InvalidWorkspace", "Invalid workspace");
			return false;
		}

		// If the workspace name autodetected and is the same as the host we should assume that no valid workspace was found.
		// TODO: Note that in this case the above call to ::TestClientConnection should have already failed so we could consider removing this check
		if (InSettings.Workspace.IsEmpty() && OutSettings.Workspace == TO_TCHAR(TestP4.GetHost().Text(), bIsUnicodeServer))
		{
			OutConnectionErrors.ErrorMessage = LOCTEXT("P4ErrorConnection_MissingWorkspace", "Missing workspace");
			OutConnectionErrors.AdditionalErrors.Add(LOCTEXT("P4ErrorConnection_NoWorkspaceFound", "No workspace was found for the current user"));
			return false;
		}
	}
	else
	{
		OutSettings.Workspace = InSettings.Workspace;
	}

	//whether successful or not, disconnect to clean up
	TestP4.Final(&P4Error);
	
	if (P4Error.Test())
	{
		OutConnectionErrors.ErrorMessage = LOCTEXT("P4_FailedDisconnect", "P4ERROR: Failed to disconnect from revision control server.");

		StrBuf ErrorMessage;
		P4Error.Fmt(&ErrorMessage);

		OutConnectionErrors.AdditionalErrors.Add(FText::FromString(TO_TCHAR(ErrorMessage.Text(), bIsUnicodeServer)));
		return false;
	}

	return true;
}

bool FPerforceConnection::EnsureValidConnection(const FPerforceConnectionInfo& InSettings, FPerforceSourceControlProvider& SCCProvider, EConnectionOptions Options,
	FPerforceConnectionInfo& OutSettings, ISourceControlProvider::FInitResult::FConnectionErrors& OutConnectionErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceConnection::EnsureValidConnection);

	bool bResult = EnsureValidConnectionInternal(InSettings, SCCProvider, Options, OutSettings, OutConnectionErrors);

	if (!bResult)
	{
		FinalizeErrors(SCCProvider, OutSettings, OutConnectionErrors);

		if (!EnumHasAllFlags(Options, EConnectionOptions::SuppressErrorLogging))
		{
			FTSMessageLog SourceControlLog("SourceControl");

			SourceControlLog.Error(OutConnectionErrors.ErrorMessage);
			for (const FText& ErrorMsg : OutConnectionErrors.AdditionalErrors)
			{
				SourceControlLog.Error(ErrorMsg);
			}
		}

		SCCProvider.SetLastErrors(OutConnectionErrors.AdditionalErrors);
	}

	return bResult;
}

bool FPerforceConnection::GetWorkspaces(const FPerforceConnectionInfo& InConnectionInfo, FOnIsCancelled InOnIsCancelled, TArray<FScWorkspaceInfo>& OutWorkspaceInfos, FSourceControlResultInfo& OutResultInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceConnection::GetWorkspaces)

	if (bEstablishedConnection)
	{
		TArray<FString> Params;
		bool bAllowWildHosts = !GIsBuildMachine;
		Params.Add(TEXT("-u"));
		Params.Add(InConnectionInfo.UserName);

		FP4RecordSet Records;
		bool bConnectionDropped = false;
		bool bCommandOK = RunCommand(TEXT("clients"), Params, Records, OutResultInfo, InOnIsCancelled, bConnectionDropped);

		if (bCommandOK)
		{
			FString LocalHostName = InConnectionInfo.HostOverride;
			if (LocalHostName.Len() == 0)
			{
				// No host override, check environment variable
				LocalHostName = FPlatformMisc::GetEnvironmentVariable(TEXT("P4HOST"));
			}

			if (LocalHostName.Len() == 0)
			{
				// no host name, use local machine name
				LocalHostName = FString(FPlatformProcess::ComputerName()).ToLower();
			}
			else
			{
				LocalHostName = LocalHostName.ToLower();
			}

			for (const FP4Record& ClientRecord : Records)
			{
				const FString& ClientName = ClientRecord(TEXT("client"));
				const FString& HostName = ClientRecord(TEXT("Host"));
				const FString& RootDir = ClientRecord(TEXT("Root"));

				//this clientspec has to be meant for this machine ( "" hostnames mean any host can use ths clientspec in p4 land)
				bool bHostNameMatches = (LocalHostName == HostName.ToLower());
				bool bHostNameWild = (HostName.Len() == 0);

				if (bHostNameMatches || (bHostNameWild && bAllowWildHosts))
				{
					FScWorkspaceInfo& WorkspaceInfo = OutWorkspaceInfos.Emplace_GetRef();
					WorkspaceInfo.Name = ClientName;
					WorkspaceInfo.RootDir = RootDir;
				}
				else
				{
					UE_LOGF(LogSourceControl, Verbose, " %ls client specs rejected due to host name mismatch (%ls)", *ClientName, *HostName);
				}
			}
		}

		return bCommandOK;
	}

	return false;
}

bool FPerforceConnection::GetProjectWorkspaces(const FPerforceConnectionInfo& InConnectionInfo, FOnIsCancelled InOnIsCancelled, TArray<FString>& OutWorkspaceNames, FSourceControlResultInfo& OutResultInfo)
{
	TArray<FScWorkspaceInfo> WorkspaceInfos;
	if (GetWorkspaces(InConnectionInfo, MoveTemp(InOnIsCancelled), WorkspaceInfos, OutResultInfo))
	{
		FString ProjectDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ISourceControlModule::Get().GetSourceControlProjectDir()).ToLower();
		ProjectDir = ProjectDir.Replace(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);

		for (FScWorkspaceInfo& WorkspaceInfo : WorkspaceInfos)
		{
			WorkspaceInfo.RootDir.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);

			// A workspace root could be "null" which allows the user to map depot locations to different drives.
			// Allow these workspaces since we already allow workspaces mapped to drive letters.
			const bool bIsNullClientRootPath = (WorkspaceInfo.RootDir == TEXT("null"));

			if (!WorkspaceInfo.RootDir.EndsWith(TEXT("/")))
			{
				WorkspaceInfo.RootDir += TEXT("/");
			}

			// Only allow paths that are ACTUALLY legit for this application
			if (bIsNullClientRootPath || FPathViews::IsParentPathOf(WorkspaceInfo.RootDir, ProjectDir))
			{
				OutWorkspaceNames.Emplace(MoveTemp(WorkspaceInfo.Name));
			}
			else
			{
				UE_LOGF(LogSourceControl, Verbose, " %ls client specs rejected due to root directory mismatch (%ls)", *WorkspaceInfo.Name, *WorkspaceInfo.RootDir);
			}
		}

		return true;
	}
	return false;
}

bool FPerforceConnection::IsValidConnection()
{
	return bEstablishedConnection && !P4Client.Dropped();
}

bool FPerforceConnection::IsIgnored(const FString& InFile)
{
	if (Ignore* P4Ignore = P4Client.GetIgnore())
	{
		const StrPtr& IgnoreFile = P4Client.GetIgnoreFile();
		const StrBuf File(FROM_TCHAR(*InFile, bIsUnicode));

		return P4Ignore->Reject(File, IgnoreFile, nullptr) != 0;
	}

	return false;
}

void FPerforceConnection::Disconnect()
{
	Error P4Error;

	P4Client.Final(&P4Error);

	if (P4Error.Test())
	{
		StrBuf ErrorMessage;
		P4Error.Fmt(&ErrorMessage);
		UE_LOGF(LogSourceControl, Error, "P4ERROR: Failed to disconnect from Server.");
		UE_LOGF(LogSourceControl, Error, "%ls", TO_TCHAR(ErrorMessage.Text(), bIsUnicode));
	}
}

double FPerforceConnection::GetLatestCommuncationTime() const
{
	return LatestCommunicateTime;
}

bool FPerforceConnection::RunCommand(	const FString& InCommand, const TArray<FString>& InParameters, FP4RecordSet& OutRecordSet, 
                                        TArray<FSharedBuffer>* OutData, FSourceControlResultInfo& OutResultInfo,
                                        FOnIsCancelled InIsCancelled, bool& OutConnectionDropped, ERunCommandFlags RunFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FPerforceConnection::RunCommand_%s"), *InCommand));

	if (!bEstablishedConnection || bDebugBreakConnection)
	{
		return false;
	}

	const bool bLogCommandDetails = !EnumHasAllFlags(RunFlags, ERunCommandFlags::DisableCommandLogging);

	FString FullCommand = InCommand;

	// Prepare arguments
	int32 ArgC = InParameters.Num();
	UTF8CHAR** ArgV = new UTF8CHAR*[ArgC];
	for (int32 Index = 0; Index < ArgC; Index++)
	{
		if(bIsUnicode)
		{
			FTCHARToUTF8 UTF8String(*InParameters[Index]);
			ArgV[Index] = new UTF8CHAR[UTF8String.Length() + 1];
			FMemory::Memcpy(ArgV[Index], UTF8String.Get(), UTF8String.Length() + 1);
		}
		else
		{
			ArgV[Index] = new UTF8CHAR[InParameters[Index].Len() + 1];
			FMemory::Memcpy(ArgV[Index], TCHAR_TO_ANSI(*InParameters[Index]), InParameters[Index].Len() + 1);
		}
		
		if (bLogCommandDetails)
		{
			FullCommand += TEXT(" ");
			FullCommand += InParameters[Index];
		}
	}

	if (bLogCommandDetails)
	{
		UE_LOGF( LogSourceControl, Log, "Attempting 'p4 %ls'", *FullCommand );
	}

	double SCCStartTime = FPlatformTime::Seconds();

	P4Client.SetArgv(ArgC, (char**)ArgV);

	FP4KeepAlive KeepAlive(InIsCancelled);
	P4Client.SetBreak(&KeepAlive);

	OutRecordSet.Reset();

	EP4ClientUserFlags ClientUserFlags = EP4ClientUserFlags::None;
	ClientUserFlags |= bIsUnicode ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;
	ClientUserFlags |= OutData != nullptr ? EP4ClientUserFlags::CollectData : EP4ClientUserFlags::None;
	
	FP4ClientUser User(OutRecordSet, ClientUserFlags, OutResultInfo);
	if (EnumHasAllFlags(RunFlags, ERunCommandFlags::Quiet))
	{
		User.SetQuiet();
	}

	P4Client.Run(FROM_TCHAR(*InCommand, bIsUnicode), &User);
	if ( P4Client.Dropped() )
	{
		OutConnectionDropped = true;
	}

	if (OutData != nullptr)
	{
		*OutData = User.ReleaseData();
	}	

	P4Client.SetBreak(nullptr);

	// Free arguments
	for (int32 Index = 0; Index < ArgC; Index++)
	{
		delete [] ArgV[Index];
	}
	delete [] ArgV;

	// Only report connection related errors to avoid clearing of connection related error messages
	if (InCommand != TEXT("info"))
	{
		SCCProvider.SetLastErrors(OutResultInfo.ErrorMessages);
	}

	if (bLogCommandDetails)
	{
		const double ExecutionTime = FPlatformTime::Seconds() - SCCStartTime;
		if (ExecutionTime >= 0.1)
		{
			UE_LOGF(LogSourceControl, Log, "P4 execution time: %0.4f seconds. Command: %ls", ExecutionTime, *FullCommand);
		}
		else
		{
			UE_LOGF(LogSourceControl, VeryVerbose, "P4 execution time: %0.4f seconds. Command: %ls", ExecutionTime, *FullCommand);
		}
	}

	if (!OutConnectionDropped)
	{
		LatestCommunicateTime = FPlatformTime::Seconds();
	}

	return OutRecordSet.Num() > 0;
}

int32 FPerforceConnection::CreatePendingChangelist(const FText &Description, const TArray<FString>& Files, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo)
{
	TArray<FString> Params;
	FP4RecordSet Records;
	EP4ClientUserFlags Flags = bIsUnicode ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;

	FP4KeepAlive KeepAlive(InIsCancelled);
	P4Client.SetBreak(&KeepAlive);

	FP4CreateChangelistClientUser User(Records, Flags, OutResultInfo, Description, Files, P4Client);
	P4Client.Run("change", &User);

	P4Client.SetBreak(nullptr);

	return User.ChangelistNumber;
}

int32 FPerforceConnection::EditPendingChangelist(const FText& NewDescription, int32 ChangelistNumber, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo)
{
	FP4RecordSet PreviousRecords;

	// Get changelist current specification
	{
		TArray<FString> Params;
		bool bConnectionDropped = false;
		const ERunCommandFlags RunFlags = ERunCommandFlags::DisableCommandLogging;

		Params.Add(TEXT("-o"));
		// TODO : make this work also for default changelist, but should really be a Create
		Params.Add(FString::Printf(TEXT("%d"), ChangelistNumber));

		if (!RunCommand(TEXT("change"), Params, PreviousRecords, nullptr, OutResultInfo, InIsCancelled, bConnectionDropped, RunFlags))
		{
			return 0;
		}
	}

	// Update description with the new description
	{
		TArray<FString> Params;
		FP4RecordSet Records;
		EP4ClientUserFlags Flags = bIsUnicode ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;

		const char* ArgV[] = { "-i" };
		P4Client.SetArgv(1, const_cast<char* const*>(ArgV));

		FP4KeepAlive KeepAlive(InIsCancelled);
		P4Client.SetBreak(&KeepAlive);

		FP4EditChangelistClientUser User(Records, Flags, OutResultInfo, NewDescription, ChangelistNumber, PreviousRecords, P4Client);
		P4Client.Run("change", &User);

		P4Client.SetBreak(nullptr);

		return (!OutResultInfo.HasErrors() ? User.ChangelistNumber : 0);
	}
}

bool FPerforceConnection::CreateWorkspace(FStringView WorkspaceSpec, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceConnection::CreateWorkspace);

	TArray<FString> Params;
	FP4RecordSet Records;

	EP4ClientUserFlags Flags = bIsUnicode ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;

	const char* ArgV[] = { "-i" };
	P4Client.SetArgv(1, const_cast<char* const*>(ArgV));

	FP4KeepAlive KeepAlive(InIsCancelled);
	P4Client.SetBreak(&KeepAlive);

	FP4CommandWithStdInputClientUser User(WorkspaceSpec, Records, Flags, OutResultInfo, P4Client);
	User.SetQuiet();	// p4 client does not return tagged output, so any output messages will be
						// printed to stdout. Setting this will prevent that.

	P4Client.Run("client", &User);

	P4Client.SetBreak(nullptr);
	
	return !OutResultInfo.HasErrors();
}

void FPerforceConnection::EstablishConnection(const FPerforceConnectionInfo& InConnectionInfo, FSourceControlResultInfo* OutResultInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceConnection::EstablishConnection);

	// Verify Input. ServerName and UserName are required
	if ( InConnectionInfo.Port.IsEmpty() || InConnectionInfo.UserName.IsEmpty() )
	{
		return;
	}

	//Connection assumed successful
	bEstablishedConnection = true;

	UE_LOGF(LogSourceControl, Verbose, "Attempting P4 connection: %ls/%ls", *InConnectionInfo.Port, *InConnectionInfo.UserName);

	P4Client.SetProg("UE");
	P4Client.SetProtocol("tag", "");
	P4Client.SetProtocol("enableStreams", "");

	//Set configuration based params
	P4Client.SetPort(TCHAR_TO_ANSI(*InConnectionInfo.Port));

	Error P4Error;
	if(InConnectionInfo.HostOverride.Len() > 0)
	{
		UE_LOGF(LogSourceControl, Verbose, " ... overriding host");
		P4Client.SetHost(TCHAR_TO_ANSI(*InConnectionInfo.HostOverride));
	}

	UE_LOGF(LogSourceControl, Verbose, " ... connecting");

	//execute the connection to perforce using the above settings
	P4Client.Init(&P4Error);

	//ensure the connection is valid
	UE_LOGF(LogSourceControl, Verbose, " ... validating connection");
	if (P4Error.Test())
	{
		bEstablishedConnection = false;
		StrBuf ErrorMessage;
		P4Error.Fmt(&ErrorMessage);

		if (OutResultInfo != nullptr)
		{
			OutResultInfo->ErrorMessages.Add(FText::FromString(TEXT("P4ERROR: Invalid connection to server.")));
			OutResultInfo->ErrorMessages.Add(FText::FromString(ANSI_TO_TCHAR(ErrorMessage.Text())));
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "P4ERROR: Invalid connection to server.");
			UE_LOGF(LogSourceControl, Error, "%ls", ANSI_TO_TCHAR(ErrorMessage.Text()));
		}
	}
	else
	{
		TArray<FString> Params;
		FSourceControlResultInfo DummyResultInfo;
		FP4RecordSet Records;
		bool bConnectionDropped = false;
		const ERunCommandFlags RunFlags = ERunCommandFlags::DisableCommandLogging;

		UE_LOGF(LogSourceControl, Verbose, " ... checking unicode status");

		if (RunCommand(TEXT("info"), Params, Records, nullptr, DummyResultInfo, FOnIsCancelled(), bConnectionDropped, RunFlags))
		{
			// Get character encoding
			bIsUnicode = Records[0].Find(TEXT("unicode")) != nullptr;
			if(bIsUnicode)
			{
				P4Client.SetTrans(CharSetApi::UTF_8);
				UE_LOGF(LogSourceControl, Verbose, " server is unicode");
			}

			// Now we know our unicode status we can gather the client root
			P4Client.SetUser(FROM_TCHAR(*InConnectionInfo.UserName, bIsUnicode));

			if (InConnectionInfo.Ticket.Len() > 0)
			{
				P4Client.SetPassword(FROM_TCHAR(*InConnectionInfo.Ticket, bIsUnicode));
			}
			else if (InConnectionInfo.Password.Len() > 0)
			{
				Login(InConnectionInfo);
			}


			if (InConnectionInfo.Workspace.Len())
			{
				P4Client.SetClient(FROM_TCHAR(*InConnectionInfo.Workspace, bIsUnicode));
			}

			P4Client.SetCwd(FROM_TCHAR(*FPaths::RootDir(), bIsUnicode));

			// Gather the client root
			UE_LOGF(LogSourceControl, Verbose, " ... getting info");
			bConnectionDropped = false;
			if (RunCommand(TEXT("info"), Params, Records, nullptr, DummyResultInfo, FOnIsCancelled(), bConnectionDropped, RunFlags))
			{
				UE_LOGF(LogSourceControl, Verbose, " ... getting clientroot");
				ClientRoot = Records[0](TEXT("clientRoot"));

				// Make sure that workspace roots end with a seperator or are null
				if (ClientRoot != TEXT("null"))
				{
					FPaths::NormalizeFilename(ClientRoot);		// Don't use NormalizeDirectoryName, we want to preserve trailing slashes
					FPaths::RemoveDuplicateSlashes(ClientRoot); // Perforce client specs support duplicate slashes

					if (!ClientRoot.EndsWith(TEXT("/")))
					{
						ClientRoot.Append(TEXT("/"));
					}
				}
			}
		}
	}
}

FString FPerforceConnection::GetUser()
{
	if (bEstablishedConnection)
	{
		return TO_TCHAR(P4Client.GetUser().Text(), bIsUnicode);
	}
	else
	{
		return FString();
	}
}

FScopedPerforceConnection::FScopedPerforceConnection( FPerforceSourceControlCommand& InCommand )
	: Connection(nullptr)
	, Concurrency(InCommand.Concurrency)
{
	FSourceControlResultInfo* ResultInfo = InCommand.Operation->ShouldLogErrors() ? nullptr : &InCommand.ResultInfo;

	Initialize(InCommand.ConnectionInfo, InCommand.Worker->GetSCCProvider(), ResultInfo);
	if (IsValid())
	{
		InCommand.MarkConnectionAsSuccessful();
	}
}

FScopedPerforceConnection::FScopedPerforceConnection(EConcurrency::Type InConcurrency, FPerforceSourceControlProvider& SCCProvider)
	: Connection(nullptr)
	, Concurrency(InConcurrency)
{
	Initialize(SCCProvider.AccessSettings().GetConnectionInfo(), SCCProvider, nullptr);
}

void FScopedPerforceConnection::Initialize( const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& SCCProvider, FSourceControlResultInfo* OutResultInfo)
{
	if(Concurrency == EConcurrency::Synchronous)
	{
		// Synchronous commands reuse the same persistent connection to reduce
		// the number of expensive connection attempts.
		if (SCCProvider.EstablishPersistentConnection(OutResultInfo) )
		{
			Connection = SCCProvider.GetPersistentConnection();
		}
	}
	else
	{
		// Async commands form a new connection for each attempt because
		// using the persistent connection is not threadsafe
		FPerforceConnection* NewConnection = new FPerforceConnection(InConnectionInfo, SCCProvider, OutResultInfo);
		if ( NewConnection->IsValidConnection() )
		{
			Connection = NewConnection;
		}
	}
}

FScopedPerforceConnection::~FScopedPerforceConnection()
{
	if(Connection != nullptr)
	{
		if(Concurrency == EConcurrency::Asynchronous)
		{
			// Remove this connection as it is only temporary
			Connection->Disconnect();
			delete Connection;
		}
		
		Connection = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
