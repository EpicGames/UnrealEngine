// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PerforceSourceControlPrivate.h"
#include "PerforceSourceControlCommand.h"
#include "Memory/SharedBuffer.h"

class FPerforceSourceControlProvider;
struct FScWorkspaceInfo;

/** A map containing result of running Perforce command */
class FP4Record : public TMap<FString, FString>
{
public:
	virtual ~FP4Record() = default;

	const FString& operator()(const FString& Key) const
	{
		if (const FString* Found = Find(Key))
		{
			return *Found;
		}
		return EmptyStr;
	}

	const FString& operator()(const TCHAR* Key) const
	{
		if (const FString* Found = FindByHash(FCrc::Strihash_DEPRECATED(Key), Key))
		{
			return *Found;
		}
		return EmptyStr;
	}

private:
	static const FString EmptyStr;
};

typedef TArray<FP4Record> FP4RecordSet;

/** Options to be used with FPerforceConnection::EnsureValidConnection */
enum class EConnectionOptions : uint8
{
	/** No options are applied */
	None				= 0,
	/** The connection does not require a workspace to be considered valid */
	WorkspaceOptional	= 1 << 0,
	/** Errors will not be logged but will still be returned to the caller */
	SuppressErrorLogging	= 1 << 1
};
ENUM_CLASS_FLAGS(EConnectionOptions);

/** Optional flags that can be passed in when a command is run */
enum class ERunCommandFlags : uint32
{
	/** No special options selected */
	Default					= 0,
	/** Prevents us writing the full perforce command (once evaluated) to the log file */
	DisableCommandLogging		= 1 << 0,
	/** The same as passing -q to a perforce command as part of the g-opts. Prevents info output from being printed to stdout */
	Quiet					= 1 << 1
};
ENUM_CLASS_FLAGS(ERunCommandFlags);

class FPerforceConnection
{
public:
	//This constructor is strictly for internal questions to perforce (get client spec list, etc)
	// When OutResultInfo is set, errors encountered will be added to it and when not set (nullptr) errors will be logged immediately
	FPerforceConnection(const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& InSCCProvider, FSourceControlResultInfo* OutResultInfo);
	/** API Specific close of source control project*/
	~FPerforceConnection();

	/** 
	 * Attempts to automatically detect the workspace to use based on the working directory
	 */
	static bool AutoDetectWorkspace(const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& SCCProvider, FString& OutWorkspaceName, TArray<FText>& OutErrorMessages);

	/**
	 * Set up a connection to the server with the given credentials. The function can attempt to autodetect missing credentials or fix incorrect ones with
	 * the final credentials being returned to the caller.
	 * 
	 * @param InSettings			The initial connection credentials.
	 * @param SCCProvider			The provider that is setting up the connection.
	 * @param Options				Used to specialize initialization behavior, @see EConnectionOptions.
	 * @param OutSettings			The finalized connection credentials. If the connection failed then this will contain the credentials that were 
	 *								used for the step that failed.
	 * @param OutConnectionErrors	A collection of errors encountered.
	 * 
	 * @return - True if a valid connection was established, otherwise false.
	 */
	static bool EnsureValidConnection(	const FPerforceConnectionInfo& InSettings, FPerforceSourceControlProvider& SCCProvider, EConnectionOptions Options,
										FPerforceConnectionInfo& OutSettings, ISourceControlProvider::FInitResult::FConnectionErrors& OutConnectionErrors);

	/**
	 * Get all Perforce workspaces for the current host
	 * @param InConnectionInfo	Connection credentials
	 * @param InOnIsCancelled	Delegate called to check for operation cancellation.
	 * @param OutWorkspaceInfos	Workspace infos output.
	 * @param OutErrorMessages	Any error messages output.
	 * @return - True if successful
	 */
	bool GetWorkspaces(const FPerforceConnectionInfo& InConnectionInfo, FOnIsCancelled InOnIsCancelled, TArray<FScWorkspaceInfo>& OutWorkspaceInfos, FSourceControlResultInfo& OutResultInfo);

	/**
	 * Get Perforce workspaces under which of the current project is located
	 * @param InConnectionInfo	Connection credentials
	 * @param InOnIsCancelled	Delegate called to check for operation cancellation.
	 * @param OutWorkspaceNames	Workspace names output.
	 * @param OutErrorMessages	Any error messages output.
	 * @return - True if successful
	 */
	bool GetProjectWorkspaces(const FPerforceConnectionInfo& InConnectionInfo, FOnIsCancelled InOnIsCancelled, TArray<FString>& OutWorkspaceNames, FSourceControlResultInfo& OutResultInfo);

	/** Returns true if connection is currently active */
	bool IsValidConnection();

	/** Returns true if file is ignored through p4ignore */
	bool IsIgnored(const FString& InFile);

	/** If the connection is valid, disconnect from the server */
	void Disconnect();

	/** Returns the platform time of the last time the connection successfully communicated with the server */
	double GetLatestCommuncationTime() const;

	/**
	 * Runs internal perforce command, catches exceptions, returns results
	 */
	bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, FP4RecordSet& OutRecordSet, FSourceControlResultInfo& OutResultInfo, FOnIsCancelled InIsCancelled, bool& OutConnectionDropped)
	{
		return RunCommand(InCommand, InParameters, OutRecordSet, nullptr, OutResultInfo, InIsCancelled, OutConnectionDropped);
	}

	/**
	 * Runs internal perforce command, catches exceptions, returns results
	 */
	bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, FP4RecordSet& OutRecordSet, TArray<FSharedBuffer>* OutData, FSourceControlResultInfo& OutResultInfo, FOnIsCancelled InIsCancelled, bool& OutConnectionDropped, ERunCommandFlags RunFlags = ERunCommandFlags::Default);

	/**
	 * Creates a changelist with the specified description
	 */
	int32 CreatePendingChangelist(const FText &Description, const TArray<FString>& InFiles, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo);

	/**
	 * Edits a changelist with a new description
	 */
	int32 EditPendingChangelist(const FText& NewDescription, int32 ChangelistNumber, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo);

	/** 
	 * Creates a workspace based on the spec provided via the WorkspaceSpec.
	 * 
	 * @param WorkspaceSpec		The specification of the workspace to create
	 * @param InIsCancelled		Delegate allowing the canceling of the command if needed
	 * @param OutResultInfo		Struct that will end up containing info and error messages for the operation
	 * 
	 * @return Returns true if no errors were encountered, otherwise false
	 */
	bool CreateWorkspace(FStringView WorkspaceSpec, FOnIsCancelled InIsCancelled, FSourceControlResultInfo& OutResultInfo);

	/**
	 * Attempt to login - some servers will require this 
	 * @param	InConnectionInfo		Credentials to use
	 * @return true if successful
	 */
	bool Login(const FPerforceConnectionInfo& InConnectionInfo);

	/**
	 * Make a valid connection if possible
	 * @param InConnectionInfo		Connection credentials
	 * @param OutResultInfo Optional structure to store errors encountered while establishing the connection.
	 *                      When set to nullptr, errors will be immediately logged.
	 */
	void EstablishConnection(const FPerforceConnectionInfo& InConnectionInfo, FSourceControlResultInfo* OutResultInfo);

	/** 
	 * Return the user of the connection
	 * 
	 * @return The current perforce user, this will be blank if the connect failed.
	 */
	FString GetUser();

public:
	
	/** Perforce API client object */
	ClientApi P4Client;

	/** The current root for the client workspace */
	FString ClientRoot;

	/** true if connection was successfully established */
	bool bEstablishedConnection;

	/** Is this a connection to a unicode server? */ 
	bool bIsUnicode;

	/** A record of the last time the connection was used */
	double LatestCommunicateTime;

private:
	/** The source control provider that the connection is working from */
	FPerforceSourceControlProvider& SCCProvider;
};

/**
 * Connection that is used within specific scope
 */
class FScopedPerforceConnection
{
public:
	/** 
	 * Constructor - establish a connection.
	 * The concurrency of the passed in command determines whether the persistent connection is 
	 * used or another connection is established (connections cannot safely be used across different
	 * threads).
	 */
	FScopedPerforceConnection( class FPerforceSourceControlCommand& InCommand);

	/** 
	 * Constructor - establish a connection.
	 * The concurrency passed in determines whether the persistent connection is used or another 
	 * connection is established (connections cannot safely be used across different threads).
	 */
	FScopedPerforceConnection( EConcurrency::Type InConcurrency, FPerforceSourceControlProvider& SCCProvider);

	/**
	 * Destructor - disconnect if this is a temporary connection
	 */
	~FScopedPerforceConnection();

	/** Get the connection wrapped by this class */
	FPerforceConnection& GetConnection() const
	{
		return *Connection;
	}

	/** Check if this connection is valid */
	bool IsValid() const
	{
		return Connection != NULL;
	}

private:
	/** Set up the connection */
	void Initialize(const FPerforceConnectionInfo& InConnectionInfo, FPerforceSourceControlProvider& SCCProvider, FSourceControlResultInfo* OutResultInfo);

private:
	/** The perforce connection we are using */
	FPerforceConnection* Connection;

	/** The concurrency of this connection */
	EConcurrency::Type Concurrency;
};
