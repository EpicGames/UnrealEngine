// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Helper struct defining parameters we need to establish a connection */
struct FPerforceConnectionInfo
{
	/** The host and port for your Perforce server. Usage ServerName:1234 */
	FString Port;

	/** Perforce username */
	FString UserName;

	/** Perforce workspace */
	FString Workspace;

	/** Perforce host override - not usually set */
	FString HostOverride;

	/** Ticket if in use */
	FString Ticket;

	/** Password if in use */
	FString Password;

	/** Changelist number */
	FString ChangelistNumber;

	/** Use P4USER, P4PORT and P4CLIENT from P4 environment if set*/
	bool bUseP4Config = false;
};
