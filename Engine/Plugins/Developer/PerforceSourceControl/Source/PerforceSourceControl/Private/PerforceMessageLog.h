// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "ISourceControlModule.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"

/** 
 * A thread safe replacement for FMessageLog which can be called from background threads. 
 * It only exposes methods from FMessageLog that we would be able to safely delay, such 
 * as messages. We do not provide any functionality to open error dialogs etc.
 * At the moment if we detect a message is being queued when not on the game thread we log
 * it instead of sending to the FMessageLog system. In the future we will store the messages
 * and marshal them to the GameThread so that they can be displayed as originally intended.
 */
class FTSMessageLog
{
public:
	FTSMessageLog() = delete;
	FTSMessageLog(const FName& InLogName)
		: Log(InLogName)
	{}

	FTSMessageLog(FTSMessageLog&&) = default;
	FTSMessageLog& operator = (FTSMessageLog&&) = default;

	FTSMessageLog(const FTSMessageLog&) = delete;
	FTSMessageLog& operator = (const FTSMessageLog&) = delete;

	~FTSMessageLog() = default;

	TSharedRef<FTokenizedMessage> Message(EMessageSeverity::Type InSeverity, const FText& InMessage = FText())
	{
		if (IsInGameThread())
		{
			return Log.Message(InSeverity, InMessage);
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity, InMessage);
			UE_LOGF(LogSourceControl, Display, "%ls", *Message->ToText().ToString());

			return Message;
		}
	}

	TSharedRef<FTokenizedMessage> Error(const FText& InMessage = FText())
	{
		if (IsInGameThread())
		{
			return Log.Error(InMessage);
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, InMessage);
			UE_LOGF(LogSourceControl, Error, "%ls", *Message->ToText().ToString());

			return Message;
		}
	}

	TSharedRef<FTokenizedMessage> PerformanceWarning(const FText& InMessage = FText())
	{
		if (IsInGameThread())
		{
			return Log.PerformanceWarning(InMessage);
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::PerformanceWarning, InMessage);
			UE_LOGF(LogSourceControl, Warning, "%ls", *Message->ToText().ToString());

			return Message;
		}
	}

	TSharedRef<FTokenizedMessage> Warning(const FText& InMessage = FText())
	{
		if (IsInGameThread())
		{
			return Log.Warning(InMessage);
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, InMessage);
			UE_LOGF(LogSourceControl, Warning, "%ls", *Message->ToText().ToString());

			return Message;
		}
	}

	TSharedRef<FTokenizedMessage> Info(const FText& InMessage = FText())
	{
		if (IsInGameThread())
		{
			return Log.Info(InMessage);
		}
		else
		{
			TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, InMessage);
			UE_LOGF(LogSourceControl, Display, "%ls", *Message->ToText().ToString());

			return Message;
		}
	}

private:

	FMessageLog Log;
};
