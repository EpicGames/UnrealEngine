// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlChangelist.h"

static const int32 DEFAULT_CHANGELIST_NUMBER = -1;
const FPerforceSourceControlChangelist FPerforceSourceControlChangelist::DefaultChangelist(DEFAULT_CHANGELIST_NUMBER);

FPerforceSourceControlChangelist::FPerforceSourceControlChangelist()
	: ChangelistNumber(DEFAULT_CHANGELIST_NUMBER)
	, bInitialized(false)
{
}