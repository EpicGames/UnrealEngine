// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlChangelist.h"

class FPerforceSourceControlChangelist : public ISourceControlChangelist
{
public:
	FPerforceSourceControlChangelist();

	explicit FPerforceSourceControlChangelist(int32 InChangelistNumber)
		: ChangelistNumber(InChangelistNumber)
		, bInitialized(true)
	{
	}

	virtual bool CanDelete() const override
	{
		return !IsDefault();
	}

	bool operator==(const FPerforceSourceControlChangelist& InOther) const
	{
		return ChangelistNumber == InOther.ChangelistNumber;
	}

	bool operator!=(const FPerforceSourceControlChangelist& InOther) const
	{
		return ChangelistNumber != InOther.ChangelistNumber;
	}

	virtual bool IsDefault() const override
	{
		return ChangelistNumber == DefaultChangelist.ChangelistNumber;
	}

	bool IsInitialized() const
	{
		return bInitialized;
	}

	void Reset()
	{
		ChangelistNumber = DefaultChangelist.ChangelistNumber;
		bInitialized = false;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FPerforceSourceControlChangelist& PerforceChangelist)
	{
		return GetTypeHash(PerforceChangelist.ChangelistNumber);
	}

	FString ToString() const
	{
		return ChangelistNumber != DefaultChangelist.ChangelistNumber ? FString::FromInt(ChangelistNumber) : TEXT("default");
	}

	int32 ToInt() const
	{
		return ChangelistNumber;
	}

	virtual FString GetIdentifier() const override
	{
		return ToString();
	}

public:
	static const FPerforceSourceControlChangelist DefaultChangelist;

private:
	int32 ChangelistNumber;
	bool bInitialized;
};

typedef TSharedRef<class FPerforceSourceControlChangelist, ESPMode::ThreadSafe> FPerforceSourceControlChangelistRef;