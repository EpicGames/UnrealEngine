// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlChangelistState.h"

#define LOCTEXT_NAMESPACE "PerforceSourceControl.ChangelistState"

FPerforceSourceControlChangelistState::FPerforceSourceControlChangelistState(const FPerforceSourceControlChangelist& InChangelist)
	: Changelist(InChangelist)
	, bHasShelvedFiles(false)
{
}

FName FPerforceSourceControlChangelistState::GetIconName() const
{
	// Mimic P4V colors, returning the red icon if there are active file(s), the blue if the changelist is empty or all the files are shelved.
	return Files.Num() > 0 ? FName("SourceControl.Changelist") : FName("SourceControl.ShelvedCHangelist");
}

FName FPerforceSourceControlChangelistState::GetSmallIconName() const
{
	return GetIconName();
}

FText FPerforceSourceControlChangelistState::GetDisplayText() const
{
	return FText::FromString(Changelist.ToString());
}

FText FPerforceSourceControlChangelistState::GetDescriptionText() const
{
	return FText::FromString(Description);
}

FText FPerforceSourceControlChangelistState::GetDisplayTooltip() const
{
	return LOCTEXT("Tooltip", "Tooltip");
}

const FDateTime& FPerforceSourceControlChangelistState::GetTimeStamp() const
{
	return TimeStamp;
}

const TArray<FSourceControlStateRef> FPerforceSourceControlChangelistState::GetFilesStates() const
{
	return Files;
}

int32 FPerforceSourceControlChangelistState::GetFilesStatesNum() const
{
	return Files.Num();
}

const TArray<FSourceControlStateRef> FPerforceSourceControlChangelistState::GetShelvedFilesStates() const
{
	TArray<FSourceControlStateRef> Result;
	ShelvedFiles.GenerateValueArray(Result);
	return Result;
}

int32 FPerforceSourceControlChangelistState::GetShelvedFilesStatesNum() const
{
	return ShelvedFiles.Num();
}

FSourceControlChangelistRef FPerforceSourceControlChangelistState::GetChangelist() const
{
	FPerforceSourceControlChangelistRef ChangelistCopy = MakeShareable( new FPerforceSourceControlChangelist(Changelist));
	return StaticCastSharedRef<ISourceControlChangelist>(ChangelistCopy);
}

#undef LOCTEXT_NAMESPACE