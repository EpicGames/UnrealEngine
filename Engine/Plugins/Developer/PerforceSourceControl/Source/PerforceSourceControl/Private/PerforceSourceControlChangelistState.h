// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlChangelistState.h"

#include "PerforceSourceControlChangelist.h"

class FPerforceSourceControlChangelistState : public ISourceControlChangelistState
{
public:
	FPerforceSourceControlChangelistState(const FPerforceSourceControlChangelist& InChangelist);

	/**
	 * Get the name of the icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetIconName() const override;

	/**
	 * Get the name of the small icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetSmallIconName() const override;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDisplayText() const override;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDescriptionText() const override;

	/**
	 * Returns whether the change list description can be saved and persisted. Some changelists are
	 * special and the description cannot be persisted with the changelist. For example the P4 default
	 * changelist description cannot be persisted and the user must create a new changelist.
	 */
	virtual bool SupportsPersistentDescription() const override { return !Changelist.IsDefault(); }

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	virtual FText GetDisplayTooltip() const override;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	virtual const FDateTime& GetTimeStamp() const override;

	virtual const TArray<FSourceControlStateRef> GetFilesStates() const override;
	virtual int32 GetFilesStatesNum() const override;

	virtual const TArray<FSourceControlStateRef> GetShelvedFilesStates() const override;
	virtual int32 GetShelvedFilesStatesNum() const override;

	virtual FSourceControlChangelistRef GetChangelist() const override;

public:
	FPerforceSourceControlChangelist Changelist;

	FString Description;
	bool bHasShelvedFiles;

	TArray<FSourceControlStateRef> Files;
	TMap<FString, FSourceControlStateRef> ShelvedFiles;

	/** The timestamp of the last update */
	FDateTime TimeStamp;
};
