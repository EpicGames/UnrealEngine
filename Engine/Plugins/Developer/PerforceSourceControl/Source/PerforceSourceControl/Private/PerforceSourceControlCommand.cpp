// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlCommand.h"

#include "IPerforceSourceControlWorker.h"
#include "PerforceSourceControlChangelist.h"
#include "PerforceSourceControlProvider.h"
#include "PerforceSourceControlSettings.h"

FPerforceSourceControlCommand::FPerforceSourceControlCommand(const TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe>& InOperation, const TSharedRef<class IPerforceSourceControlWorker, ESPMode::ThreadSafe>& InWorker, const FSourceControlOperationComplete& InOperationCompleteDelegate )
	: Operation(InOperation)
	, Worker(InWorker)
	, OperationCompleteDelegate(InOperationCompleteDelegate)
	, bExecuteProcessed(0)
	, bCancelled(0)
	, bConnectionWasSuccessful(0)
	, bCancelledWhileTryingToConnect(0)
	, bCommandSuccessful(false)
	, bConnectionDropped(false)
	, bAutoDelete(true)
	, Concurrency(EConcurrency::Synchronous)
{
	// Grab the providers settings here, so we don't access them once the worker thread is launched
	check(IsInGameThread() || InOperation->CanBeCalledFromBackgroundThreads());
	ConnectionInfo = Worker->GetSCCProvider().AccessSettings().GetConnectionInfo();
}

bool FPerforceSourceControlCommand::DoWork()
{
	bCommandSuccessful = Worker->Execute(*this);
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);

	return bCommandSuccessful;
}

void FPerforceSourceControlCommand::Abandon()
{
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);
}

void FPerforceSourceControlCommand::DoThreadedWork()
{
	Concurrency = EConcurrency::Asynchronous;
	DoWork();
}

void FPerforceSourceControlCommand::Cancel()
{
	FPlatformAtomics::InterlockedExchange(&bCancelled, 1);
}

void FPerforceSourceControlCommand::MarkConnectionAsSuccessful()
{
	FPlatformAtomics::InterlockedExchange(&bConnectionWasSuccessful, 1);
}

void FPerforceSourceControlCommand::CancelWhileTryingToConnect()
{
	FPlatformAtomics::InterlockedExchange(&bCancelledWhileTryingToConnect, 1);
}

bool FPerforceSourceControlCommand::IsCanceled() const
{
	return bCancelled != 0;
}

bool FPerforceSourceControlCommand::WasConnectionSuccessful() const
{
	return bConnectionWasSuccessful != 0;
}

bool FPerforceSourceControlCommand::WasCanceledWhileTryingToConnect() const
{
	return bCancelledWhileTryingToConnect != 0;
}

ECommandResult::Type FPerforceSourceControlCommand::ReturnResults()
{
	// Save any messages that have accumulated
	Operation->AppendResultInfo(ResultInfo);

	// run the completion delegate if we have one bound
	ECommandResult::Type Result = bCancelled ? ECommandResult::Cancelled : (bCommandSuccessful ? ECommandResult::Succeeded : ECommandResult::Failed);
	OperationCompleteDelegate.ExecuteIfBound(Operation, Result);

	return Result;
}

