// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#include "PerforceConnectionInfo.h"
#include "PerforceSourceControlChangelist.h"
#include "Misc/IQueuedWork.h"

DECLARE_DELEGATE_RetVal(bool, FOnIsCancelled);

/**
 * Used to execute Perforce commands multi-threaded.
 */
class FPerforceSourceControlCommand : public IQueuedWork
{
public:

	FPerforceSourceControlCommand(const TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe>& InOperation, const TSharedRef<class IPerforceSourceControlWorker, ESPMode::ThreadSafe>& InWorker, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * This is where the real thread work is done. All work that is done for
	 * this queued object should be done from within the call to this function.
	 */
	bool DoWork();

	/**
	 * Tells the queued work that it is being abandoned so that it can do
	 * per object clean up as needed. This will only be called if it is being
	 * abandoned before completion. NOTE: This requires the object to delete
	 * itself using whatever heap it was allocated in.
	 */
	virtual void Abandon() override;

	/**
	 * This method is also used to tell the object to cleanup but not before
	 * the object has finished it's work.
	 */
	virtual void DoThreadedWork() override;

	/** Attempt to cancel the operation */
	void Cancel();

	/** Mark the connection to the server as successful */
	void MarkConnectionAsSuccessful();

	/** Mark as canceled while trying to connect */
	void CancelWhileTryingToConnect();

	/** Is the operation canceled? */
	bool IsCanceled() const;

	/** Was the connection to the server successful? */
	bool WasConnectionSuccessful() const;

	/** Was the operation canceled while trying to connect? */
	bool WasCanceledWhileTryingToConnect() const;

	/** Save any results and call any registered callbacks. */
	ECommandResult::Type ReturnResults();

public:
	/** Connection parameters, reproduced here because if is not safe to access the provider's settings from another thread */
	FPerforceConnectionInfo ConnectionInfo;

	/** Operation we want to perform - contains outward-facing parameters & results */
	TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe> Operation;

	/** The object that will actually do the work */
	TSharedRef<class IPerforceSourceControlWorker, ESPMode::ThreadSafe> Worker;

	/** Delegate to notify when this operation completes */
	FSourceControlOperationComplete OperationCompleteDelegate;

	/**If true, this command has been processed by the source control thread*/
	volatile int32 bExecuteProcessed;

	/**If true, this command has been cancelled*/
	volatile int32 bCancelled;

	/**If true, the source control connection was made successfully */
	volatile int32 bConnectionWasSuccessful;

	/**If true, this command was cancelled while trying to connect */
	volatile int32 bCancelledWhileTryingToConnect;

	/**If true, the source control command succeeded*/
	bool bCommandSuccessful;

	/**If true, the source control connection was dropped while this command was being executed*/
	bool bConnectionDropped;

	/** If true, this command will be automatically cleaned up in Tick() */
	bool bAutoDelete;

	/** Whether we are running multi-treaded or not*/
	EConcurrency::Type Concurrency;

	/** Files to perform this operation on */
	TArray< FString > Files;

	/** Changelist to perform this operation on */
	FPerforceSourceControlChangelist Changelist;

	/** Potential error, warning and info message storage */
	FSourceControlResultInfo ResultInfo;

	/** Branch names for status queries */
	TArray< FString > StatusBranchNames;

	/** Content roots for branch mapping */
	TArray<FString> ContentRoots;

};
