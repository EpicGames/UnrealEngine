// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "SourceControlOperationBase.h"
#include "UObject/NameTypes.h"

// This head contains operations that are not exposed by the public source control
// module API and so can only be called from within the perforce source control
// module itself.

/**
 * Internal-only source control operation for retrieving available workspaces for the current project
 */

class FGetProjectWorkspaces : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetProjectWorkspaces";
	}

public:
	/** Results are just an array of workspaces */
	TArray<FString> Results;
};
