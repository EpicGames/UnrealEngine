// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlLabel.h"

#include "Logging/MessageLog.h"
#include "PerforceConnection.h"
#include "PerforceSourceControlPrivate.h"
#include "PerforceSourceControlProvider.h"
#include "PerforceSourceControlRevision.h"
#include "PerforceSourceControlSettings.h"

#define LOCTEXT_NAMESPACE "PerforceSourceControl"

const FString& FPerforceSourceControlLabel::GetName() const
{
	return Name;
}	

static void ParseFilesResults(FPerforceSourceControlProvider& SCCProvider, const FP4RecordSet& InRecords, TArray< TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> >& OutRevisions)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& DepotFile = ClientRecord(TEXT("depotFile"));
		const FString& RevisionNumber = ClientRecord(TEXT("rev"));
		const FString& Date = ClientRecord(TEXT("time"));
		const FString& ChangelistNumber = ClientRecord(TEXT("change"));
		const FString& Action = ClientRecord(TEXT("action"));
		check(RevisionNumber.Len() != 0);

		// @todo: this revision is incomplete, but is sufficient for now given the usage of labels to get files, rather
		// than review revision histories.
		TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> Revision = MakeShared<FPerforceSourceControlRevision>(SCCProvider);
		Revision->FileName = DepotFile;
		Revision->RevisionNumber = FCString::Atoi(*RevisionNumber);
		Revision->ChangelistNumber = FCString::Atoi(*ChangelistNumber);
		Revision->Action = Action;
		Revision->Date = FDateTime(1970, 1, 1, 0, 0, 0, 0) + FTimespan::FromSeconds(FCString::Atoi(*Date));

		OutRevisions.Add(Revision);
	}
}

FPerforceSourceControlLabel::FPerforceSourceControlLabel(FPerforceSourceControlProvider& InSCCProvider, const FString& InName)
	: SCCProvider(InSCCProvider)
	, Name(InName)
{
}

bool FPerforceSourceControlLabel::GetFileRevisions( const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> >& OutRevisions ) const
{
	bool bCommandOK = false;

	FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, GetSCCProvider());
	if(ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FP4RecordSet Records;
		TArray<FString> Parameters;
		FSourceControlResultInfo ResultInfo;
		for(auto Iter(InFiles.CreateConstIterator()); Iter; Iter++)
		{
			Parameters.Add(*Iter + TEXT("@") + Name);
		}
		bool bConnectionDropped = false;
		bCommandOK = Connection.RunCommand(TEXT("files"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped);
		if(bCommandOK)
		{
			ParseFilesResults(GetSCCProvider(), Records, OutRevisions);
		}
		else
		{
			for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
			{
				FMessageLog("SourceControl").Error(FText::Format(LOCTEXT("GetFileRevisionsErrorFormat","GetFileRevisions Error: {0}"), ErrorMsg));
			}
		}
	}

	return bCommandOK;
}

bool FPerforceSourceControlLabel::Sync( const TArray<FString>& InFilenames ) const
{
	bool bCommandOK = false;

	FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, GetSCCProvider());
	if(ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FP4RecordSet Records;
		FSourceControlResultInfo ResultInfo;
		TArray<FString> Parameters;
		for(const auto& Filename : InFilenames)
		{
			Parameters.Add(Filename + TEXT("@") + Name);
		}
		bool bConnectionDropped = false;
		bCommandOK = Connection.RunCommand(TEXT("sync"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped);
		if(!bCommandOK)
		{
			for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
			{
				FMessageLog("SourceControl").Error(FText::Format(LOCTEXT("GetFileRevisionsSyncErrorFormat", "Sync Error: {0}"), ErrorMsg));
			}
		}
	}

	return bCommandOK;
}

#undef LOCTEXT_NAMESPACE