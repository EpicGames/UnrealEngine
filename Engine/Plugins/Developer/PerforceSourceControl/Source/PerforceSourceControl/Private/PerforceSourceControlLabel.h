// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlRevision.h"
#include "ISourceControlLabel.h"

class FPerforceSourceControlProvider;

/** 
 * Abstraction of a Perforce label.
 */
class FPerforceSourceControlLabel : public ISourceControlLabel
{
public:

	FPerforceSourceControlLabel(FPerforceSourceControlProvider& InSCCProvider, const FString& InName);
	virtual ~FPerforceSourceControlLabel() = default;

	/** ISourceControlLabel implementation */
	virtual const FString& GetName() const override;
	virtual bool GetFileRevisions( const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> >& OutRevisions ) const override;
	virtual bool Sync( const TArray<FString>& InFilenames ) const override;

private:

	/** Internal accessor to the source control provider associated with the object */
	FPerforceSourceControlProvider& GetSCCProvider() const
	{
		return SCCProvider;
	}

	/** The source control provider that this object is associated with */
	FPerforceSourceControlProvider& SCCProvider;

	/** Label name */
	FString Name;
};
