// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlModule.h"

#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "PerforceConnection.h"
#include "PerforceSourceControlOperations.h"
#include "PerforceSourceControlSettings.h"

#define LOCTEXT_NAMESPACE "PerforceSourceControl"

void FPerforceSourceControlModule::StartupModule()
{
	// Register our operations
	IPerforceSourceControlWorker::RegisterWorkers();

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature( "SourceControl", &PerforceSourceControlProvider );
}

void FPerforceSourceControlModule::ShutdownModule()
{
	// shut down the provider, as this module is going away
	PerforceSourceControlProvider.Close();

	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature( "SourceControl", &PerforceSourceControlProvider );
}

IMPLEMENT_MODULE(FPerforceSourceControlModule, PerforceSourceControl);

#undef LOCTEXT_NAMESPACE
