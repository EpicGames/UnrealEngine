// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PerforceSourceControlProvider.h"

class FPerforceSourceControlModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	/** The perforce source control provider that will be used by default via ISourceControlModule::Get().GetProvider() etc */
	FPerforceSourceControlProvider PerforceSourceControlProvider;
};
