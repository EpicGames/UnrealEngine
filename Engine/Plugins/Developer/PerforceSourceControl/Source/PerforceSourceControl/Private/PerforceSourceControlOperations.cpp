// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlOperations.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "Algo/Transform.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "ISourceControlModule.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "PerforceConnection.h"
#include "PerforceMessageLog.h"
#include "PerforceSourceControlChangelistState.h"
#include "PerforceSourceControlChangeStatusOperation.h"
#include "PerforceSourceControlCommand.h"
#include "PerforceSourceControlInternalOperations.h"
#include "PerforceSourceControlPrivate.h"
#include "PerforceSourceControlProvider.h"
#include "PerforceSourceControlRevision.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "TedsRevisionControlStateHelper.h"

#if SOURCE_CONTROL_WITH_SLATE
	#include "Textures/SlateIcon.h"
#endif

#define LOCTEXT_NAMESPACE "PerforceSourceControl"

namespace
{
DECLARE_DELEGATE_RetVal_OneParam(FPerforceSourceControlWorkerRef, FGetPerforceSourceControlWorker, FPerforceSourceControlProvider&)

static TMap<FName, FGetPerforceSourceControlWorker> WorkersMap;

bool GAlwaysBranchFilesOnCopy = false;
static FAutoConsoleVariableRef CVarAlwaysBranchFilesOnCopy(
	TEXT("p4.AlwaysBranchFilesOnCopy"),
	GAlwaysBranchFilesOnCopy,
	TEXT("Use legacy behavior of always branching a file in perforce when copying."),
	ECVF_Default
);

bool GParseIgnores = false;
static FAutoConsoleVariableRef CVarParseIgnores(
	TEXT("p4.ParseIgnores"),
	GParseIgnores,
	TEXT("Use p4ignore behavior to exclude files from revision control consideration."),
	ECVF_Default
);

}

namespace PerforceWorkersTedsUtility {
	
void UpdateEditorDataStorage(const FPerforceSourceControlState& State)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::RevisionControl;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!DataStorage)
		{
			return;
		}

		const RowHandle Row = FindOrAddPackageRow(*DataStorage, State.GetFilename());
		if (Row == InvalidRowHandle)
		{
			return;
		}

		FSccStateColumn StateColumn;

		if (State.IsSourceControlled())
		{
			// Modification priority mirrors the previous call-order: each match overwrites
			// the prior value, so IsConflicted wins over IsDeleted wins over IsAdded wins
			// over IsModified. If none match, Modification stays ESCCModification::None.
			if (State.IsModified())   StateColumn.Modification = ESCCModification::Modified;
			if (State.IsAdded())      StateColumn.Modification = ESCCModification::Added;
			if (State.IsDeleted())    StateColumn.Modification = ESCCModification::Removed;
			if (State.IsConflicted()) StateColumn.Modification = ESCCModification::Conflicted;

			const bool bIsCheckedOutByOther = State.IsCheckedOutOther(nullptr);
			if (bIsCheckedOutByOther && State.bExclusiveCheckout)
			{
				StateColumn.bIsExternallyLocked = true;
			}

			StateColumn.bIsLocked           = State.IsCheckedOut();
			StateColumn.bIsExternallyEdited = bIsCheckedOutByOther;
			StateColumn.bIsNotCurrent       = (State.LocalRevNumber < State.DepotRevNumber);
		}
		else
		{
			StateColumn.Modification = ESCCModification::NotInDepot;
		}

#if SOURCE_CONTROL_WITH_SLATE
		const FSlateIcon Icon = State.GetIcon();
		StateColumn.IconStyleSetName = Icon.GetStyleSetName();
		StateColumn.IconStyleName    = Icon.GetStyleName();
		StateColumn.IconOverlayName  = Icon.GetOverlayStyleName();
		StateColumn.DisplayTooltip   = State.GetDisplayTooltip();
#endif

		ApplySccStateUpdate(*DataStorage, Row, StateColumn);
	};
}

template<typename Type>
FPerforceSourceControlWorkerRef InstantiateWorker(FPerforceSourceControlProvider& InSourceControlProvider)
{
	return MakeShared<Type>(InSourceControlProvider);
}

void IPerforceSourceControlWorker::RegisterWorkers()
{
	WorkersMap.Add("Connect", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceConnectWorker>));
	WorkersMap.Add("CheckOut", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceCheckOutWorker>));
	WorkersMap.Add("UpdateStatus", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceUpdateStatusWorker>));
	WorkersMap.Add("MarkForAdd", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceMarkForAddWorker>));
	WorkersMap.Add("Delete", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceDeleteWorker>));
	WorkersMap.Add("Revert", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceRevertWorker>));
	WorkersMap.Add("Sync", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceSyncWorker>));
	WorkersMap.Add("CheckIn", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceCheckInWorker>));
	WorkersMap.Add("GetWorkspaces", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetWorkspacesWorker>));
	WorkersMap.Add("GetProjectWorkspaces", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetProjectWorkspacesWorker>));
	WorkersMap.Add("Copy", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceCopyWorker>));
	WorkersMap.Add("Resolve", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceResolveWorker>));
	WorkersMap.Add("ChangeStatus", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceChangeStatusWorker>));
	WorkersMap.Add("UpdateChangelistsStatus", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetPendingChangelistsWorker>));
	WorkersMap.Add("GetChangelistDetails", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetChangelistDetailsWorker>));
	WorkersMap.Add("NewChangelist", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceNewChangelistWorker>));
	WorkersMap.Add("DeleteChangelist", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceDeleteChangelistWorker>));
	WorkersMap.Add("EditChangelist", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceEditChangelistWorker>));
	WorkersMap.Add("RevertUnchanged", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceRevertUnchangedWorker>));
	WorkersMap.Add("MoveToChangelist", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceReopenWorker>));
	WorkersMap.Add("Shelve", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceShelveWorker>));
	WorkersMap.Add("Unshelve", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceUnshelveWorker>));
	WorkersMap.Add("DeleteShelved", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceDeleteShelveWorker>));
	WorkersMap.Add("DownloadFile", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceDownloadFileWorker>));
	WorkersMap.Add("CreateWorkspace", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceCreateWorkspaceWorker>));
	WorkersMap.Add("DeleteWorkspace", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceDeleteWorkspaceWorker>));
	WorkersMap.Add("GetFileList", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetFileListWorker>));
	WorkersMap.Add("GetFile", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceGetFileWorker>));
	WorkersMap.Add("GetSourceControlRevisionInfo", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceRevisionInfoWorker>));
	WorkersMap.Add("Where", FGetPerforceSourceControlWorker::CreateStatic(&InstantiateWorker<FPerforceWhereWorker>));
}

TSharedPtr<class IPerforceSourceControlWorker, ESPMode::ThreadSafe> IPerforceSourceControlWorker::CreateWorker(const FName& OperationName, FPerforceSourceControlProvider& SCCProvider)
{
	const FGetPerforceSourceControlWorker* Operation = WorkersMap.Find(OperationName);
	if (Operation != nullptr)
	{
		return Operation->Execute(SCCProvider);
	}

	return nullptr;
}

/**
 * Helper struct for RemoveRedundantErrors()
 */
struct FRemoveRedundantErrors
{
	FRemoveRedundantErrors(const FString& InFilter)
		: Filter(InFilter)
	{
	}

	bool operator()(const FText& Text) const
	{
		if(Text.ToString().Contains(Filter))
		{
			return true;
		}

		return false;
	}

	/** The filter string we try to identify in the reported error */
	FString Filter;
};

struct FBranchModification
{
	FBranchModification(const FString& InBranchName, const FString& InFileName, const FString& InAction, int32 InChangeList, int64 InModTime )
		: BranchName(InBranchName)
		, FileName(InFileName)
		, Action(InAction)
		, ChangeList(InChangeList)
		, ModTime(InModTime)
	{
	}

	FString BranchName;
	FString FileName;
	FString Action;
	int32 ChangeList;
	int64 ModTime;									  

	FString OtherUserCheckedOut;
	TArray<FString> CheckedOutBranches;
};

/** Checks if the name of an action corresponds to EPerforceState::OpenForAdd */
static bool IsAddAction(const FString& Action)
{
	return Action == TEXT("add") || Action == TEXT("move/add");
}

/** Checks if the name of an action corresponds to EPerforceState::MarkedForDelete */
static bool IsDeleteAction(const FString& Action)
{
	return Action == TEXT("delete") || Action == TEXT("move/delete") || Action == TEXT("deleted");
}

/**
 * Remove redundant errors (that contain a particular string) and also
 * update the commands success status if all errors were removed.
 */
static void RemoveRedundantErrors(FPerforceSourceControlCommand& InCommand, const FString& InFilter, bool bMoveToInfo = true)
{
	bool bFoundRedundantError = false;
	for(auto Iter(InCommand.ResultInfo.ErrorMessages.CreateConstIterator()); Iter; Iter++)
	{
		// Perforce reports files that are already synced as errors, so copy any errors
		// we get to the info list in this case
		if(Iter->ToString().Contains(InFilter))
		{
			if (bMoveToInfo)
			{
				InCommand.ResultInfo.InfoMessages.Add(*Iter);
			}
			bFoundRedundantError = true;
		}
	}

	InCommand.ResultInfo.ErrorMessages.RemoveAll( FRemoveRedundantErrors(InFilter) );

	// if we have no error messages now, assume success!
	if(bFoundRedundantError && InCommand.ResultInfo.ErrorMessages.Num() == 0 && !InCommand.bCommandSuccessful)
	{
		InCommand.bCommandSuccessful = true;
	}
}

static void RemoveRedundantInfo(FPerforceSourceControlCommand& InCommand, const FString& InFilter, bool bMoveToError = true)
{
	bool bFoundRedundantInfo = false;

	if (bMoveToError)
	{
		for (const FText& Message : InCommand.ResultInfo.InfoMessages)
		{
			// Perforce reports files that are already synced as errors, so copy any errors
			// we get to the info list in this case
			if (Message.ToString().Contains(InFilter))
			{
				InCommand.ResultInfo.ErrorMessages.Add(Message);
			}
		}
	}

	bFoundRedundantInfo = InCommand.ResultInfo.InfoMessages.RemoveAll(FRemoveRedundantErrors(InFilter)) > 0;

	// If we found info that we moved to be an error we should mark the command as failed.
	if (bFoundRedundantInfo && bMoveToError)
	{
		InCommand.bCommandSuccessful = false;
	}
}

/**
 * Occasionally some error messages returned by the server can be a bit misleading depending on the context
 * of the original command. This utility allows us to replace an error that we feel is misleading with a
 * better version.
 *
 * If the misleading error is found we don't just replace that exact string, we remove everything from the
 * string, starting at the misleading error and then append the correct error on the end. This is because
 * the error messages can often contain variable info (such as the port or client) which can be hard to
 * match the misleading error string against.
 */
static void ReplaceErrors(FPerforceSourceControlCommand& InCommand, const TCHAR* InMisleadingError, const TCHAR* InCorrectedError, bool bMoveToInfo)
{
	TArray<FText>& ErrorMessages = InCommand.ResultInfo.ErrorMessages;

	for (int32 Index = 0; Index < ErrorMessages.Num(); ++Index)
	{
		const FString Error = ErrorMessages[Index].ToString();

		const int32 Pos = Error.Find(InMisleadingError);
		if (Pos != INDEX_NONE)
		{
			TStringBuilder<128> NewError;
			NewError << FStringView(Error).Left(Pos) << InCorrectedError << TEXT("\n");

			if (bMoveToInfo)
			{
				InCommand.ResultInfo.InfoMessages.Add(FText::FromString(NewError.ToString()));
				ErrorMessages.RemoveAt(Index);
				--Index;
			}
			else
			{
				ErrorMessages[Index] = FText::FromString(NewError.ToString());
			}
		}
	}

	if (InCommand.ResultInfo.ErrorMessages.IsEmpty())
	{
		InCommand.bCommandSuccessful = true;
	}
}

/** Simple parsing of a record set into strings, one string per record */
static void ParseRecordSet(const FP4RecordSet& InRecords, TArray<FText>& OutResults)
{
	const FString Delimiter = FString(TEXT(" "));

	for (int32 RecordIndex = 0; RecordIndex < InRecords.Num(); ++RecordIndex)
	{
		const FP4Record& ClientRecord = InRecords[RecordIndex];
		for(FP4Record::TConstIterator It = ClientRecord.CreateConstIterator(); It; ++It)
		{
			OutResults.Add(FText::FromString(It.Key() + Delimiter + It.Value()));
		}
	}
}

/** Simple parsing of a record set to update state */
static void ParseRecordSetForState(const FP4RecordSet& InRecords, TMap<FString, EPerforceState::Type>& OutResults, bool bConsiderDepotFile = false)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& ClientFileName = ClientRecord(TEXT("clientFile"));
		const FString& DepotFileName = ClientRecord(TEXT("depotFile"));
		const FString& Action = ClientRecord(TEXT("action"));

		check(ClientFileName.Len() || (bConsiderDepotFile ? DepotFileName.Len() : false));
		FString FullPath = ClientFileName.Len() ? ClientFileName : (bConsiderDepotFile ? DepotFileName : FString());
		FPaths::NormalizeFilename(FullPath);

		if(Action.Len() > 0)
		{
			if(IsAddAction(Action))
			{
				OutResults.Add(FullPath, EPerforceState::OpenForAdd);
			}
			else if(Action == TEXT("edit"))
			{
				OutResults.Add(FullPath, EPerforceState::CheckedOut);
			}
			else if(IsDeleteAction(Action))
			{
				OutResults.Add(FullPath, EPerforceState::MarkedForDelete);
			}
			else if(Action == TEXT("abandoned"))
			{
				OutResults.Add(FullPath, EPerforceState::NotInDepot);
			}
			else if(Action == TEXT("reverted"))
			{
				const FString& OldAction = ClientRecord(TEXT("oldAction"));
				if(IsAddAction(OldAction))
				{
					OutResults.Add(FullPath, EPerforceState::NotInDepot);
				}
				else if(OldAction == TEXT("edit"))
				{
					OutResults.Add(FullPath, EPerforceState::ReadOnly);
				}
				else if(IsDeleteAction(OldAction))
				{
					OutResults.Add(FullPath, EPerforceState::ReadOnly);
				}
			}
			else if(Action == TEXT("branch") || Action == TEXT("integrate"))
			{
				OutResults.Add(FullPath, EPerforceState::Branched);
			}
			else if (Action == TEXT("cleared"))
			{
				OutResults.Add(FullPath, EPerforceState::ReadOnly);
			}
		}
	}
}

static void UpdateChangelistState(FPerforceSourceControlProvider& SCCProvider, const FPerforceSourceControlChangelist& InChangelist, const TMap<FString, EPerforceState::Type>& InOperationResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::UpdateChangelistState);

	if (InChangelist.IsInitialized())
	{
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = SCCProvider.GetStateInternal(InChangelist);

		for (TMap<FString, EPerforceState::Type>::TConstIterator It(InOperationResults); It; ++It)
		{
			TSharedPtr<FPerforceSourceControlState, ESPMode::ThreadSafe> State;

			switch (It.Value())
			{
				case EPerforceState::CheckedOut:
				case EPerforceState::OpenForAdd:
				case EPerforceState::MarkedForDelete:
				case EPerforceState::Branched:
					State = SCCProvider.GetStateInternal(It.Key());
					ChangelistState->Files.Add(State.ToSharedRef());

					// Keep the changelist stored with cached file state in sync with the actual changelist that owns this file.
					State->Changelist = InChangelist;
					break;
				default:					
					break;
			}
		}
	}
}

static bool UpdateCachedStates(FPerforceSourceControlProvider& SCCProvider, const TMap<FString, EPerforceState::Type>& InResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::UpdateCachedStates);

	for(TMap<FString, EPerforceState::Type>::TConstIterator It(InResults); It; ++It)
	{
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = SCCProvider.GetStateInternal(It.Key());
		State->SetState(It.Value());
		State->TimeStamp = FDateTime::Now();

		PerforceWorkersTedsUtility::UpdateEditorDataStorage(*State);
	}

	return InResults.Num() > 0;
}

static bool CheckWorkspaceRecordSet(const FP4RecordSet& InRecords, TArray<FText>& OutErrorMessages, FText& OutNotificationText)
{
	FString ApplicationPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ISourceControlModule::Get().GetSourceControlProjectDir()).ToLower();
	ApplicationPath = ApplicationPath.Replace(TEXT("\\"), TEXT("/"));

	for(const auto& Record : InRecords)
	{
		FString Root = Record(TEXT("Root"));

		// A workspace root could be "null" which allows the user to map depot locations to different drives.
		// Allow these workspaces since we already allow workspaces mapped to drive letters.
		const bool bIsNullClientRootPath = (Root == TEXT("null"));

		// Sanitize root name
		Root = Root.Replace(TEXT("\\"), TEXT("/"));
		if (!Root.EndsWith(TEXT("/")))
		{
			Root += TEXT("/");
		}

		if (bIsNullClientRootPath || FPathViews::IsParentPathOf(Root, ApplicationPath))
		{
			return true;
		}
		else
		{
			const FString& Client = Record(TEXT("Client"));
			OutNotificationText = FText::Format(LOCTEXT("WorkspaceError", "Workspace '{0}' with root '{1}' does not map into this project's directory '{2}'"), FText::FromString(Client), FText::FromString(Root), FText::FromString(ApplicationPath));
			OutErrorMessages.Add(OutNotificationText);
			OutErrorMessages.Add(LOCTEXT("WorkspaceHelp", "You should set your workspace up to map to a directory at or above the project's directory."));
		}
	}

	return false;
}

static bool AppendChangelistParameter(FPerforceSourceControlProvider& SCCProvider, TArray<FString>& InOutParams)
{
	FPerforceSourceControlSettings& Settings = SCCProvider.AccessSettings();

	const FString& ChangelistNumber = Settings.GetChangelistNumber();
	if ( !ChangelistNumber.IsEmpty() )
	{
		InOutParams.Add(TEXT("-c"));
		InOutParams.Add(ChangelistNumber);

		return true;
	}

	return false;
}

FName FPerforceConnectWorker::GetName() const
{
	return "Connect";
}

bool FPerforceConnectWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceConnectWorker::Execute);

	GetSCCProvider().ResetPersistentConnection();

	if (InCommand.ConnectionInfo.Workspace.IsEmpty())
	{
		// Running FConnect for perforce does not actually connect to the server but
		// validates that the clientspec for the connection is correct and working.
		// So if the connection does not have a valid clientspec then we might as well
		// early out at this point and return false.

		InCommand.bCommandSuccessful = false;
		return InCommand.bCommandSuccessful;
	}

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;
		FP4RecordSet Records;
		Parameters.Add(TEXT("-o"));
		Parameters.Add(InCommand.ConnectionInfo.Workspace);
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("client"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

		// If there are error messages, user name is most likely invalid. Otherwise, make sure workspace actually
		// exists on server by checking if we have it's update date.
		InCommand.bCommandSuccessful &= InCommand.ResultInfo.ErrorMessages.Num() == 0 && Records.Num() > 0 && Records[0].Contains(TEXT("Update"));
		if (!InCommand.bCommandSuccessful && InCommand.ResultInfo.ErrorMessages.Num() == 0)
		{
			InCommand.ResultInfo.ErrorMessages.Add(LOCTEXT("InvalidWorkspace", "Invalid workspace."));
		}

		// check if we can actually work with this workspace
		if(InCommand.bCommandSuccessful)
		{
			FText Notification;
			InCommand.bCommandSuccessful = CheckWorkspaceRecordSet(Records, InCommand.ResultInfo.ErrorMessages, Notification);
			if(!InCommand.bCommandSuccessful)
			{
				check(InCommand.Operation->GetName() == GetName());
				TSharedRef<FConnect, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FConnect>(InCommand.Operation);
				Operation->SetErrorText(Notification);
			}
			
			GetSCCProvider().SetIsWorkspaceValidForProject(InCommand.bCommandSuccessful);
		}

		if(InCommand.bCommandSuccessful)
		{
			ParseRecordSet(Records, InCommand.ResultInfo.InfoMessages);
		}
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceConnectWorker::UpdateStates() const
{
	return true;
}

FName FPerforceCheckOutWorker::GetName() const
{
	return "CheckOut";
}

bool FPerforceCheckOutWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCheckOutWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;

		if ((!AppendChangelistParameter(GetSCCProvider(), Parameters)) && InCommand.Changelist.IsInitialized())
		{
			FString ChangelistNumber = InCommand.Changelist.ToString();

			if (!ChangelistNumber.IsEmpty())
			{
				Parameters.Add(TEXT("-c"));
				Parameters.Add(ChangelistNumber);
				InChangelist = InCommand.Changelist;
			}
		}

		Parameters.Append(InCommand.Files);
		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("edit"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseRecordSetForState(Records, OutResults);
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceCheckOutWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCheckOutWorker::UpdateStates);

	// If files have been checkedout directly to a CL, modify the cached state to reflect it.
	UpdateChangelistState(GetSCCProvider(), InChangelist, OutResults);

	return UpdateCachedStates(GetSCCProvider(), OutResults);
}

FName FPerforceCheckInWorker::GetName() const
{
	return "CheckIn";
}

static FText ParseSubmitResults(const FP4RecordSet& InRecords)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& SubmittedChange = ClientRecord(TEXT("submittedChange"));
		if(SubmittedChange.Len() > 0)
		{
			return FText::Format(LOCTEXT("SubmitMessage", "Submitted changelist {0}"), FText::FromString(SubmittedChange));
		}
	}

	return LOCTEXT("SubmitMessageUnknown", "Submitted changelist");
}

static bool RunReopenCommand(FPerforceSourceControlCommand& InCommand, const TArray<FString>& InFiles, const FPerforceSourceControlChangelist& InChangelist, TArray<FString>* OutReopenedFiles = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::RunReopenCommand);

	bool bCommandSuccessful = true;

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		// Batch reopen into multiple commands, to avoid command line limits
		const int32 BatchedCount = 100;

		if (OutReopenedFiles != nullptr)
		{
			OutReopenedFiles->Reserve(InFiles.Num());
		}

		for (int32 StartingIndex = 0; StartingIndex < InFiles.Num() && bCommandSuccessful; StartingIndex += BatchedCount)
		{
			FP4RecordSet Records;
			TArray<FString> ReopenParams;

			//Add changelist information to params
			ReopenParams.Add(TEXT("-c"));
			ReopenParams.Add(InChangelist.ToString());

			int32 NextIndex = FMath::Min(StartingIndex + BatchedCount, InFiles.Num());

			for (int32 FileIndex = StartingIndex; FileIndex < NextIndex; FileIndex++)
			{
				ReopenParams.Add(InFiles[FileIndex]);
			}

			bCommandSuccessful = Connection.RunCommand(TEXT("reopen"), ReopenParams, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
			if (bCommandSuccessful && OutReopenedFiles != nullptr)
			{
				for (int32 FileIndex = StartingIndex; FileIndex < NextIndex; FileIndex++)
				{
					OutReopenedFiles->Add(InFiles[FileIndex]);
				}
			}
		}
	}

	return bCommandSuccessful;
}

static bool RemoveFilesFromChangelist(const TMap<FString, EPerforceState::Type>& Results, TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe>& ChangelistState)
{
	return ChangelistState->Files.RemoveAll([&Results](FSourceControlStateRef& State) -> bool
		{
			return Results.Contains(State->GetFilename());
		}) > 0;
}

static bool RemoveFilesFromChangelist(const TMap<FString, EPerforceState::Type>& Results, FPerforceSourceControlProvider& SCCProvider, const FPerforceSourceControlChangelist& Changelist)
{
	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = SCCProvider.GetStateInternal(Changelist);
	return RemoveFilesFromChangelist(Results, ChangelistState);
}

bool FPerforceCheckInWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCheckInWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FText CachedDescription;
		
		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCheckIn>(InCommand.Operation);

		TArray<FString> FilesToSubmit = InCommand.Files;

		FPerforceSourceControlChangelist ChangeList(InCommand.Changelist);
		FSourceControlChangelistStateRef ChangelistState = GetSCCProvider().GetStateInternal(InCommand.Changelist);
		TArray<FString> ReopenedFiles;

		InCommand.bCommandSuccessful = true;

		// Adds changelist's files to FilesToSubmit so we can update their state after submit
		if ((FilesToSubmit.Num() == 0) && (InCommand.Changelist.IsInitialized()))
		{
			Algo::Transform(ChangelistState->GetFilesStates(), FilesToSubmit, [](const FSourceControlStateRef& FileState)
			{
				return FileState->GetFilename();
			});
		}

		if (InCommand.Changelist.IsDefault())
		{
			int32 NewChangeList = Connection.CreatePendingChangelist(Operation->GetDescription(), TArray<FString>(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
			if (NewChangeList > 0)
			{
				ChangeList = FPerforceSourceControlChangelist(NewChangeList);
				InCommand.bCommandSuccessful = RunReopenCommand(InCommand, FilesToSubmit, ChangeList, &ReopenedFiles);
			}
			else
			{
				InCommand.bCommandSuccessful = false;
			}
		}
		else if (!Operation->GetDescription().IsEmpty())
		{
			// Retrieves cached description to restore it in case of submit failure.
			CachedDescription = ChangelistState->GetDescriptionText();

			int32 UpdatedChangelistNumber = Connection.EditPendingChangelist(Operation->GetDescription(), InCommand.Changelist.ToInt(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
			InCommand.bCommandSuccessful = (UpdatedChangelistNumber != 0) && (UpdatedChangelistNumber == InCommand.Changelist.ToInt());
		}

		// Only submit if reopen was successful (when starting from the default changelist) or always otherwise
		if (InCommand.bCommandSuccessful)
		{
			TArray<FString> SubmitParams;
			FP4RecordSet Records;

			if (Operation->GetKeepCheckedOut())
			{
				SubmitParams.Add(TEXT("-r"));
			}

			SubmitParams.Add(TEXT("-c"));
			SubmitParams.Add(ChangeList.ToString());

			InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("submit"), SubmitParams, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

			if (InCommand.ResultInfo.HasErrors())
			{
				InCommand.bCommandSuccessful = false;
			}

			const bool bMoveToError = true;
			RemoveRedundantInfo(InCommand, TEXT("- warning: file not mapped in stream"), bMoveToError);

			if (InCommand.bCommandSuccessful)
			{
				// Remove any deleted files from status cache
				TArray<TSharedRef<ISourceControlState, ESPMode::ThreadSafe>> States;
				GetSCCProvider().GetState(FilesToSubmit, States, EStateCacheUsage::Use);
				for (const TSharedRef<ISourceControlState>& State : States)
				{
					if (State->IsDeleted())
					{
						GetSCCProvider().RemoveFileFromCache(State->GetFilename());
					}

					if (Operation->GetKeepCheckedOut() && (State->IsAdded() || State->IsCheckedOut()))
					{
						FilesToKeepCheckedOut.Emplace(State->GetFilename(), EPerforceState::CheckedOut);
					}
				}

				StaticCastSharedRef<FCheckIn>(InCommand.Operation)->SetSuccessMessage(ParseSubmitResults(Records));

				for (const FString& Filename : FilesToSubmit)
				{
					OutResults.Add(Filename, EPerforceState::ReadOnly);
				}

				InChangelist = InCommand.Changelist;
				OutChangelist = ChangeList;
			}
		}

		if (!InCommand.bCommandSuccessful)
		{
			// If the submit failed, clean up the changelist created above
			if (InCommand.Changelist.IsDefault())
			{
				// Reopen the assets to the default changelist to remove them from the changelist we created above
				if (ReopenedFiles.Num() > 0)
				{
					RunReopenCommand(InCommand, ReopenedFiles, InCommand.Changelist);
				}

				// Delete the changelist we created above
				{
					FP4RecordSet Records;
					TArray<FString> ChangeParams;
					ChangeParams.Add(TEXT("-d"));
					ChangeParams.Add(ChangeList.ToString());
					Connection.RunCommand(TEXT("change"), ChangeParams, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
				}
			}
			// If we modified description and we have the previous one, restore it.
			else if ((!Operation->GetDescription().IsEmpty()) && (!CachedDescription.IsEmpty()))
			{
				Connection.EditPendingChangelist(CachedDescription, InCommand.Changelist.ToInt(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceCheckInWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCheckInWorker::UpdateStates);

	bool bUpdatedStates = UpdateCachedStates(GetSCCProvider(), OutResults);
	bool bUpdatedChangelistStates = false;

	if(!OutChangelist.IsDefault()) // e.g. operation succeeded
	{
		// Delete changelist, whether its a temporary one or not
		bUpdatedChangelistStates = GetSCCProvider().RemoveChangelistFromCache(OutChangelist);

		// If it's a temporary one, then remove the submitted files from the default changelist
		if (InChangelist.IsDefault())
		{
			bUpdatedChangelistStates = RemoveFilesFromChangelist(OutResults, GetSCCProvider(), InChangelist);
		}

		// Ensure to update the cache with respect to the file kept in checkout. Those files are move to the 'default' changelist by P4.
		if (!FilesToKeepCheckedOut.IsEmpty())
		{
			UpdateChangelistState(GetSCCProvider(), FPerforceSourceControlChangelist::DefaultChangelist, FilesToKeepCheckedOut);
			UpdateCachedStates(GetSCCProvider(), FilesToKeepCheckedOut);
			bUpdatedChangelistStates = true;
		}
	}

	return (bUpdatedStates || bUpdatedChangelistStates);
}

FName FPerforceGetFileListWorker::GetName() const
{
	return "GetFileList";
}

static void AppendMaskParameter(TArray<FString>& InOutParams)
{
	for (int i = 0; i < InOutParams.Num(); ++i)
	{
		InOutParams[i] += TEXT("/...");
	}
}

bool FPerforceGetFileListWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetFileListWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TSharedRef<FGetFileList, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetFileList>(InCommand.Operation);

		TArray<FString> Parameters;
		if (!Operation->GetIncludeDeleted())
		{
			Parameters.Add(TEXT("-e"));
		}
		Parameters.Append(InCommand.Files);
		if (Operation->GetMethodUsed() == FGetFileList::FolderSearch)
		{
			AppendMaskParameter(Parameters);
		}
		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("files"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		if (Operation->ShouldBeQuiet())
		{
			RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."), false);
		}
		bool bConsiderDepotFiles = Operation->GetMethodUsed() == FGetFileList::FileRegexSearch;
		ParseRecordSetForState(Records, OutResults, bConsiderDepotFiles);

		TArray<FString> FilesList;
		FilesList.Reserve(OutResults.Num());
		for (const auto& Pair : OutResults)
		{
			FilesList.Add(Pair.Key);
		}
		Operation->SetFilesList(MoveTemp(FilesList));
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceGetFileListWorker::UpdateStates() const
{
	return false;
}

FName FPerforceMarkForAddWorker::GetName() const
{
	return "MarkForAdd";
}

bool FPerforceMarkForAddWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceMarkForAddWorker::Execute);

	// Avoid invalid p4 syntax if there's no file to process
	if (InCommand.Files.IsEmpty())
	{
		return true;
	}

	// Perforce will allow you to mark files for add that don't currently exist on disk
	// This goes against the workflow of our other SCC providers (such as SVN and Git),
	// so we manually check that the files exist before allowing this command to continue
	// This keeps the behavior consistent between SCC providers
	bool bHasMissingFiles = false;
	for(const FString& FileToAdd : InCommand.Files)
	{
		if(!IFileManager::Get().FileExists(*FileToAdd))
		{
			bHasMissingFiles = true;
			InCommand.ResultInfo.ErrorMessages.Add(FText::Format(LOCTEXT("Error_FailedToMarkFileForAdd_FileMissing", "Failed mark the file '{0}' for add. The file doesn't exist on disk."), FText::FromString(FileToAdd)));
		}
	}
	if(bHasMissingFiles)
	{
		InCommand.bCommandSuccessful = false;
		return false;
	}

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;
		FP4RecordSet Records;

		if (InCommand.Changelist.IsInitialized())
		{
			FString ChangelistNumber = InCommand.Changelist.ToString();

			if (!ChangelistNumber.IsEmpty())
			{
				Parameters.Add(TEXT("-c"));
				Parameters.Add(ChangelistNumber);
				InChangelist = InCommand.Changelist;
			}
		}

		AppendChangelistParameter(GetSCCProvider(), Parameters);
		Parameters.Append(InCommand.Files);
	
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("add"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseRecordSetForState(Records, OutResults);

		const bool bMoveToError = true;
		RemoveRedundantInfo(InCommand, TEXT("- ignored file can't be added."), bMoveToError);
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceMarkForAddWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceMarkForAddWorker::UpdateStates);

	UpdateChangelistState(GetSCCProvider(), InChangelist, OutResults);

	return UpdateCachedStates(GetSCCProvider(), OutResults);
}

FName FPerforceDeleteWorker::GetName() const
{
	return "Delete";
}

bool FPerforceDeleteWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;

		if ((!AppendChangelistParameter(GetSCCProvider(), Parameters)) && InCommand.Changelist.IsInitialized())
		{
			FString ChangelistNumber = InCommand.Changelist.ToString();

			if (!ChangelistNumber.IsEmpty())
			{
				Parameters.Add(TEXT("-c"));
				Parameters.Add(ChangelistNumber);
				Changelist = InCommand.Changelist;
			}
		}

		Parameters.Append(InCommand.Files);

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("delete"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseRecordSetForState(Records, OutResults);
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceDeleteWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteWorker::UpdateStates);

	// If files have been checkedout directly to a CL, modify the cached state to reflect it.
	UpdateChangelistState(GetSCCProvider(), Changelist, OutResults);

	return UpdateCachedStates(GetSCCProvider(), OutResults);
}

FName FPerforceRevertWorker::GetName() const
{
	return "Revert";
}

bool FPerforceRevertWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceRevertWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;

		if (InCommand.Changelist.IsInitialized())
		{
			Parameters.Add(TEXT("-c"));
			Parameters.Add(InCommand.Changelist.ToString());
		}
		else
		{
			AppendChangelistParameter(GetSCCProvider(), Parameters);
		}

		TSharedRef<FRevert> RevertOperation = StaticCastSharedRef<FRevert>(InCommand.Operation);

		if (RevertOperation->IsSoftRevert())
		{
			Parameters.Add(TEXT("-k"));
		}
		else if (RevertOperation->ShouldDeleteNewFiles())
		{
			Parameters.Add(TEXT("-w"));
		}

		if (InCommand.Files.Num() != 0)
		{
			Parameters.Append(InCommand.Files);
		}
		else if(InCommand.Changelist.IsInitialized()) // Note: safety net here, as we probably never want to revert everything
		{
			Parameters.Add(TEXT("//..."));
		}

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("revert"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseRecordSetForState(Records, OutResults);

		for (TMap<FString, EPerforceState::Type>::TConstIterator It(OutResults); It; ++It)
		{
			if (It->Value == EPerforceState::MarkedForDelete)
			{
				RevertOperation->AddDeletedFile(It->Key);
			}
		}

		ChangelistToUpdate = InCommand.Changelist;
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceRevertWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceRevertWorker::UpdateStates);

	bool bUpdatedCachedStates = UpdateCachedStates(GetSCCProvider(), OutResults);
	bool bUpdatedChangelists = false;

	bUpdatedChangelists = ChangelistToUpdate.IsInitialized() && RemoveFilesFromChangelist(OutResults, GetSCCProvider(), ChangelistToUpdate);

	// Use reverted files to update changelist state.
	for (TMap<FString, EPerforceState::Type>::TConstIterator It(OutResults); It; ++It)
	{
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = GetSCCProvider().GetStateInternal(It.Key());

		bUpdatedChangelists |= State->Changelist.IsInitialized() && RemoveFilesFromChangelist(OutResults, GetSCCProvider(), State->Changelist);
	}

	return bUpdatedCachedStates || bUpdatedChangelists;
}

FName FPerforceSyncWorker::GetName() const
{
	return "Sync";
}

static void ParseSyncResults(const FP4RecordSet& InRecords, TMap<FString, EPerforceState::Type>& OutResults)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& FileName = ClientRecord(TEXT("clientFile"));
		const FString& Action = ClientRecord(TEXT("action"));

		check(FileName.Len());
		FString FullPath(FileName);
		FPaths::NormalizeFilename(FullPath);

		if (Action.Len() > 0)
		{
			if (Action == TEXT("updated") || (Action == TEXT("refreshed")))
			{
				OutResults.Add(FullPath, EPerforceState::ReadOnly);
			}
		}
	}
}

bool FPerforceSyncWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceSyncWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		
		TSharedRef<FSync, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FSync>(InCommand.Operation);
		TArray<FString> Parameters;

		if (Operation->IsForced())
		{
			Parameters.Add(TEXT("-f"));
		}

		const FString& Revision = Operation->GetRevision();

		// check for directories and add '...'
		for (FString FileName : InCommand.Files)
		{
			if (FileName.EndsWith(TEXT("/")))
			{
				FileName += TEXT("...");
			}

			if (Operation->IsHeadRevisionFlagSet())
			{
				FileName += TEXT("#head");
			}
			else if (Operation->IsLastSyncedFlagSet())
			{
				TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = GetSCCProvider().GetStateInternal(FileName);

				if (State->LocalRevNumber == ISourceControlState::INVALID_REVISION)
				{
					const FString ErrorMsg = FString::Format(TEXT("Cached state of {0} is missing. Please refresh and try again."), {FileName});
					
					InCommand.ResultInfo.ErrorMessages.Add(FText::FromString(ErrorMsg));
					InCommand.bCommandSuccessful = false;
					
					return InCommand.bCommandSuccessful;
				}

				FileName += FString::Format(TEXT("#{0}"), { State->LocalRevNumber });
			}
			else if (!Revision.IsEmpty())
			{
				FileName += FString::Printf(TEXT("#%s"), *Revision);
			}

			Parameters.Add(MoveTemp(FileName));
		}

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("sync"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseSyncResults(Records, OutResults);

		RemoveRedundantErrors(InCommand, TEXT("file(s) up-to-date"));
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceSyncWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceSyncWorker::UpdateStates);

	return UpdateCachedStates(GetSCCProvider(), OutResults);
}

static void ParseBranchModificationResults(const FP4RecordSet& InRecords, const TArray<FText>& ErrorMessages, const TArray<FString>& ContentRoots, TMap<FString, FBranchModification>& BranchModifications)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::ParseBranchModificationResults);

	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& DepotFileName = ClientRecord(TEXT("depotFile"));
		const FString& ClientFileName = ClientRecord(TEXT("clientFile"));
		const FString& HeadAction = ClientRecord(TEXT("headAction"));
		int64 HeadModTime = FCString::Atoi64(*ClientRecord(TEXT("headModTime")));
		int64 HeadTime = FCString::Atoi64(*ClientRecord(TEXT("headTime")));
		int32 HeadChange = FCString::Atoi(*ClientRecord(TEXT("headChange")));

		// Filter out add modifications as these can be the result of generating a missing uasset from source content
		// and in the case where there are 2 competing adds, this is a conflict state
		if (HeadAction == TEXT("add"))
		{
			continue;
		}

		// Get the content filename and add to branch states
		FString CurrentBranch(TEXT("*CurrentBranch"));
		FString Branch, BranchFile;
		for (const FString& ContentRoot : ContentRoots)
		{
			if (DepotFileName.Split(ContentRoot, &Branch, &BranchFile))
			{
				// Sanitize names
				Branch.RemoveFromEnd(FString(TEXT("/")));
				BranchFile.RemoveFromStart(FString(TEXT("/")));

				break;
			}
		}

		if (!Branch.Len() || !BranchFile.Len())
		{
			continue;
		}

		if (ClientFileName.Len())
		{
			Branch = CurrentBranch;
		}

		// In the case of delete, P4 stores 0 for modification time, so use the HeadTime of the CL
		if (!HeadModTime)
		{
			HeadModTime = HeadTime;
		}

		// Check for modification in another branch
		if (BranchModifications.Contains(BranchFile))
		{
			FBranchModification& BranchModification = BranchModifications[BranchFile];
			
			if (BranchModification.ModTime == HeadModTime)
			{
				// Never overwrite a current branch modification with the same from a different branch
				if (BranchModification.BranchName == CurrentBranch && Branch != CurrentBranch)
				{
					continue;
				}

				// Never overwrite edit with an integrate for same mod time
				if (BranchModification.Action == TEXT("edit"))
				{
					continue;
				}
			}

			// filter deletes if file re-added. move/delete files cannot be re-added as they're bound to an add/delete
			if (HeadAction == TEXT("delete") && BranchModification.ChangeList > HeadChange)
			{
				continue;
			}

			if (BranchModification.ModTime <= HeadModTime)
			{
				BranchModification.ModTime = HeadModTime;
				BranchModification.BranchName = Branch;
				BranchModification.Action = HeadAction;
				BranchModification.ChangeList = HeadChange;
			}
		}
		else
		{
			BranchModifications.Add(BranchFile, FBranchModification(Branch, BranchFile, HeadAction, HeadChange, HeadModTime));
		}
	}
}

static void ParseUpdateStatusResults(const FP4RecordSet& InRecords, const TArray<FText>& ErrorMessages, TArray<FPerforceSourceControlState>& OutStates, const TArray<FString>& ContentRoots, TMap<FString, FBranchModification>& BranchModifications)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceUpdateStatusWorker::ParseUpdateStatusResults);

	// Build up a map of any other branch states
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& ClientFile = ClientRecord(TEXT("clientFile"));
		if (ClientFile.Len())
		{
			// Local workspace file, we're only interested in other branches here
			continue;
		}

		// Get the content filename and add to branch states
		const FString& DepotFileName = ClientRecord(TEXT("depotFile"));
		const FString& OtherOpen = ClientRecord(TEXT("otherOpen"));

		FString Branch;
		FString FileName(ClientFile);
		for (const FString& ContentRoot : ContentRoots)
		{
			if (DepotFileName.Split(ContentRoot, &Branch, &FileName))
			{
				// Sanitize
				Branch.RemoveFromEnd(FString(TEXT("/")));
				FileName.RemoveFromStart(FString(TEXT("/")));

				// Add to branch modifications if not currently recorded
				if (FileName.Len() && !BranchModifications.Contains(FileName))
				{
					BranchModifications.Add(FileName, FBranchModification(Branch, FileName, FString(TEXT("none")), 0, 0));
				}

				break;
			}
		}

		if (!FileName.Len())
		{
			// There was a problem getting the filename
			continue;
		}

		// Store checkout information to branch state
		FBranchModification& BranchModification = BranchModifications[FileName];

		if (OtherOpen.Len())
		{
			BranchModification.CheckedOutBranches.AddUnique(Branch);

			int32 OtherOpenNum = FCString::Atoi(*OtherOpen);
			for (int32 OpenIdx = 0; OpenIdx < OtherOpenNum; ++OpenIdx)
			{
				const FString OtherOpenRecordKey = FString::Printf(TEXT("otherOpen%d"), OpenIdx);
				const FString& OtherOpenRecordValue = ClientRecord(OtherOpenRecordKey);

				int32 AtIndex = OtherOpenRecordValue.Find(TEXT("@"));
				FString OtherOpenUser = AtIndex == INDEX_NONE ? FString(TEXT("")) : OtherOpenRecordValue.Left(AtIndex);
				BranchModification.OtherUserCheckedOut += OtherOpenUser + TEXT(" @ ") + Branch;

				if (OpenIdx < OtherOpenNum - 1)
				{
					BranchModification.OtherUserCheckedOut += TEXT(", ");
				}
			}
		}
	}

	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& FileName = ClientRecord(TEXT("clientFile"));
		const FString& DepotFileName = ClientRecord(TEXT("depotFile"));
		const FString& Changelist = ClientRecord(TEXT("change"));
		const FString& HeadRev  = ClientRecord(TEXT("headRev"));
		const FString& HaveRev  = ClientRecord(TEXT("haveRev"));
		const FString& OtherOpen = ClientRecord(TEXT("otherOpen"));
		const FString& OpenType = ClientRecord(TEXT("type"));
		const FString& HeadAction = ClientRecord(TEXT("headAction"));
		const FString& Action = ClientRecord(TEXT("action"));
		const FString& HeadType = ClientRecord(TEXT("headType"));
		const FString& Digest = ClientRecord(TEXT("digest"));
		const bool bUnresolved = ClientRecord.Contains(TEXT("unresolved"));

		if (!FileName.Len())
		{
			// From another branch and already encoded in the branch state map
			continue;
		}

		FString FullPath(FileName);
		FPaths::NormalizeFilename(FullPath);

		OutStates.Add(FPerforceSourceControlState(FullPath));
		FPerforceSourceControlState& State = OutStates.Last();
		State.DepotFilename = DepotFileName;

		FString Branch;
		FString BranchFile;

		for (const FString& ContentRoot : ContentRoots)
		{
			if (DepotFileName.Split(ContentRoot, &Branch, &BranchFile))
			{
				// Sanitize
				Branch.RemoveFromEnd(FString(TEXT("/")));
				BranchFile.RemoveFromStart(FString(TEXT("/")));

				break;
			}
		}

		State.State = EPerforceState::ReadOnly;
		if (Action.Len() > 0 && IsAddAction(Action))
		{
			State.State = EPerforceState::OpenForAdd;
		}
		else if (Action.Len() > 0 && IsDeleteAction(Action))
		{
			State.State = EPerforceState::MarkedForDelete;
		}
		else if (OpenType.Len() > 0)
		{
			if(Action.Len() > 0 && (Action == TEXT("branch") || Action == TEXT("integrate")))
			{
				State.State = EPerforceState::Branched;
			}
			else
			{
				State.State = EPerforceState::CheckedOut;
			}
		}
		else if (OtherOpen.Len() > 0)
		{
			// OtherOpen just reports the number of developers that have a file open, now add a string for every entry
			int32 OtherOpenNum = FCString::Atoi(*OtherOpen);
			for ( int32 OpenIdx = 0; OpenIdx < OtherOpenNum; ++OpenIdx )
			{
				const FString OtherOpenRecordKey = FString::Printf(TEXT("otherOpen%d"), OpenIdx);
				const FString& OtherOpenRecordValue = ClientRecord(OtherOpenRecordKey);

				int32 AtIndex = OtherOpenRecordValue.Find(TEXT("@"));
				FString OtherOpenUser = AtIndex == INDEX_NONE ? FString(TEXT("")) : OtherOpenRecordValue.Left(AtIndex);
				State.OtherUserCheckedOut += OtherOpenUser + TEXT(" @ ") + Branch;

				if(OpenIdx < OtherOpenNum - 1)
				{
					State.OtherUserCheckedOut += TEXT(", ");
				}
			}

			// Add to the checked out branches
			State.CheckedOutBranches.AddUnique(FEngineVersion::Current().GetBranch());

			State.State = EPerforceState::CheckedOutOther;
		}
		//file has been previously deleted, ok to add again. move/delete is not eligible for this
		else if (HeadAction.Len() > 0 && HeadAction == TEXT("delete"))
		{
			State.State = EPerforceState::NotInDepot;
		}

		if (Changelist.Len() > 0 && Changelist != TEXT("default"))
		{
			State.Changelist = FPerforceSourceControlChangelist(FCString::Atoi(*Changelist));
		}
		else
		{
			State.Changelist = FPerforceSourceControlChangelist::DefaultChangelist;
		}

		State.HeadBranch = TEXT("*CurrentBranch");
		State.HeadAction = HeadAction;
		State.HeadModTime = FCString::Atoi64(*ClientRecord(TEXT("headModTime")));
		State.HeadChangeList = FCString::Atoi(*ClientRecord(TEXT("headChange")));

		if (BranchModifications.Contains(BranchFile))
		{
			const FBranchModification& BranchModification = BranchModifications[BranchFile];

			if (BranchModification.BranchName.Len())
			{
				bool Skip = false;

				// don't record if we deleted on a status branch, though have since re-added
				if (BranchModification.Action == TEXT("delete") && BranchModification.ChangeList < State.HeadChangeList)
				{
					Skip = true;
				}

				// If the branch modification change is less recent skip it
				if (BranchModification.ModTime <= State.HeadModTime)
				{
					Skip = true;
				}
				
				if (!Skip)
				{
					State.HeadBranch = BranchModification.BranchName;
					State.HeadAction = BranchModification.Action;
					State.HeadModTime = BranchModification.ModTime;
					State.HeadChangeList = BranchModification.ChangeList;
				}
			}

			// Setup other branch check outs
			if (BranchModification.CheckedOutBranches.Num())
			{
				State.OtherUserBranchCheckedOuts += BranchModification.OtherUserCheckedOut;

				for (auto& OtherBranch : BranchModification.CheckedOutBranches)
				{
					State.CheckedOutBranches.AddUnique(OtherBranch);
				}
			}
		}

		if (HeadRev.Len() > 0 && HaveRev.Len() > 0)
		{
			TTypeFromString<int>::FromString(State.DepotRevNumber, *HeadRev);
			TTypeFromString<int>::FromString(State.LocalRevNumber, *HaveRev);
			if( bUnresolved )
			{
				int32 ResolveActionNumber = 0;
				for (;;)
				{
					// Extract the revision number
					FString VarName = FString::Printf(TEXT("resolveAction%d"), ResolveActionNumber);
					if (!ClientRecord.Contains(*VarName))
					{
						// No more revisions
						if (ResolveActionNumber == 0)
						{
							FTSMessageLog SourceControlLog("SourceControl");
							SourceControlLog.Error(FText::Format(LOCTEXT("P4Operation_ResolveWithoutAction", "Resolve is pending but no resolve actions for file {0}"), FText::FromString(FileName)));
						}
						break;
					}

					VarName = FString::Printf(TEXT("resolveBaseFile%d"), ResolveActionNumber);
					const FString& ResolveBaseFile = ClientRecord(VarName);
					VarName = FString::Printf(TEXT("resolveFromFile%d"), ResolveActionNumber);
					const FString& ResolveFromFile = ClientRecord(VarName);

					if(ResolveFromFile != ResolveBaseFile)
					{
						FTSMessageLog SourceControlLog("SourceControl");
						SourceControlLog.Error(FText::Format(LOCTEXT("P4Operation_UnsupportedCrossFileMerge", "Text cannot resolve {0} with {1}, we do not support cross file merging"), FText::FromString(ResolveBaseFile), FText::FromString(ResolveFromFile)));
						break;
					}

					VarName = FString::Printf(TEXT("resolveBaseRev%d"), ResolveActionNumber);
					const FString& ResolveBaseRev = ClientRecord(VarName);

					VarName = FString::Printf(TEXT("resolveEndFromRev%d"), ResolveActionNumber);
					const FString& ResolveEndFromRev = ClientRecord(VarName);

					State.PendingResolveInfo.BaseFile = ResolveBaseFile;
					State.PendingResolveInfo.BaseRevision = ResolveBaseRev;
					State.PendingResolveInfo.RemoteFile = ResolveFromFile;
					State.PendingResolveInfo.RemoteRevision = ResolveEndFromRev;

					++ResolveActionNumber;
				}
			}
		}

		// Check binary status
		State.bBinary = false;
		if (HeadType.Len() > 0 && HeadType.Contains(TEXT("binary")))
		{
			State.bBinary = true;
		}

		// Check exclusive checkout flag
		State.bExclusiveCheckout = false;
		if (HeadType.Len() > 0 && HeadType.Contains(TEXT("+l")))
		{
			State.bExclusiveCheckout = true;
		}
	}

	// also see if we can glean anything from the error messages
	for (const FText& Error : ErrorMessages)
	{
		//@todo P4 could be returning localized error messages
		const int32 NoSuchFilePos = Error.ToString().Find(TEXT(" - no such file(s).\n"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
		if(NoSuchFilePos != INDEX_NONE)
		{
			// found an error about a file that is not in the depot
			FString FullPath(Error.ToString().Left(NoSuchFilePos));
			FPaths::NormalizeFilename(FullPath);
			OutStates.Add(FPerforceSourceControlState(FullPath));
			FPerforceSourceControlState& State = OutStates.Last();
			State.State = EPerforceState::NotInDepot;
		}

		//@todo P4 could be returning localized error messages
		const int32 ProtectedPos = Error.ToString().Find(TEXT(" - protected namespace - access denied.\n"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
		if (ProtectedPos != INDEX_NONE)
		{
			// found an error about a file that is not in the depot
			FString FullPath(Error.ToString().Left(ProtectedPos));
			FPaths::NormalizeFilename(FullPath);
			OutStates.Add(FPerforceSourceControlState(FullPath));
			FPerforceSourceControlState& State = OutStates.Last();
			State.State = EPerforceState::NotInDepot;
		}

		//@todo P4 could be returning localized error messages
		const int32 NotUnderClientRootPos = Error.ToString().Find(TEXT("' is not under client's root"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
		if(NotUnderClientRootPos != INDEX_NONE)
		{
			// found an error about a file that is not under the client root
			static const FString Prefix(TEXT("Path \'"));
			FString FullPath(Error.ToString().Mid(Prefix.Len(), NotUnderClientRootPos - Prefix.Len()));
			FPaths::NormalizeFilename(FullPath);
			OutStates.Add(FPerforceSourceControlState(FullPath));
			FPerforceSourceControlState& State = OutStates.Last();
			State.State = EPerforceState::NotUnderClientRoot;
		}
		
		//@todo P4 could be returning localized error messages
		const int32 NotInClientViewPos = Error.ToString().Find(TEXT(" - file(s) not in client view"), ESearchCase::IgnoreCase, ESearchDir::FromStart);
		if(NotInClientViewPos != INDEX_NONE)
		{
			// found an error about a file that is not in the client view, treating it same as not under client root
			FString FullPath(Error.ToString().Left(NotInClientViewPos));
			FPaths::NormalizeFilename(FullPath);
			OutStates.Add(FPerforceSourceControlState(FullPath));
			FPerforceSourceControlState& State = OutStates.Last();
			State.State = EPerforceState::NotUnderClientRoot;
		}
	}
}

static void ParseOpenedResults(const FP4RecordSet& InRecords, const FString& ClientName, const FString& ClientRoot, TArray<FPerforceSourceControlState>& OutResults)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& ClientFileName = ClientRecord(TEXT("clientFile"));
		check(ClientFileName.Len() > 0);

		// Convert the depot file name to a local file name
		FString FullPath = ClientFileName;
		const FString PathRoot = FString::Printf(TEXT("//%s/"), *ClientName);

		if (FullPath.StartsWith(PathRoot))
		{
			const bool bIsNullClientRootPath = (ClientRoot == TEXT("null"));
			if ( bIsNullClientRootPath )
			{
				// Null clients use the pattern in PathRoot: //Workspace/FileName
				// Here we chop off the '//Workspace/' to return the workspace filename
				FullPath.RightChopInline(PathRoot.Len(), EAllowShrinking::No);
			}
			else
			{
				// This is a normal workspace where we can simply replace the pathroot with the client root to form the filename
				FullPath = FullPath.Replace(*PathRoot, *ClientRoot);
			}
		}
		else
		{
			// This file is not in the workspace, ignore it
			continue;
		}

		// Fill-in with information we got from the opened command, namely:
		// depotFile, rev, haveRev, action, change, type, user, client
		// Note: haveRev works, but we don't have the depot revision, so we might as well not write anything
		FPerforceSourceControlState& OutState = OutResults.Emplace_GetRef(FullPath);
		OutState.DepotFilename = ClientRecord(TEXT("depotFile"));

		const FString& Action = ClientRecord(TEXT("action"));
		if (Action.Len() > 0)
		{
			if(IsAddAction(Action))
			{
				OutState.State = EPerforceState::OpenForAdd;
			}
			else if(Action == TEXT("edit"))
			{
				OutState.State = EPerforceState::CheckedOut;
			}
			else if(IsDeleteAction(Action))
			{
				OutState.State = EPerforceState::MarkedForDelete;
			}
			else if (Action == TEXT("branch") || Action == TEXT("integrate"))
			{
				OutState.State = EPerforceState::Branched;
			}
		}

		const FString& Changelist = ClientRecord(TEXT("change"));
		if (Changelist.Len() > 0 && Changelist != TEXT("default"))
		{
			OutState.Changelist = FPerforceSourceControlChangelist(FCString::Atoi(*Changelist));
		}
		else
		{
			OutState.Changelist = FPerforceSourceControlChangelist::DefaultChangelist;
		}

		const FString& Type = ClientRecord(TEXT("type"));
		if (Type.Len() > 0)
		{
			OutState.bBinary = Type.Contains(TEXT("binary"));
			OutState.bExclusiveCheckout = Type.Contains(TEXT("+l"));
		}
	}
}

static void ParseOpenedResults(const FP4RecordSet& InRecords, const FString& ClientName, const FString& ClientRoot, TMap<FString, EPerforceState::Type>& OutResults)
{
	TArray<FPerforceSourceControlState> TemporaryStates;
	ParseOpenedResults(InRecords, ClientName, ClientRoot, TemporaryStates);

	for (const FPerforceSourceControlState& FileState : TemporaryStates)
	{
		if (FileState.State != EPerforceState::DontCare)
			OutResults.Add(FileState.LocalFilename, FileState.State);
	}
}

static void ParseShelvedResults(const FP4RecordSet& InRecords, TMap<FString, EPerforceState::Type>& OutResults)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& Record : InRecords)
	{
		const FString& DepotFileName = Record(TEXT("depotFile"));
		const FString& Action = Record(TEXT("action"));

		if (Action.Len() > 0 && DepotFileName.Len() > 0)
		{
			if (IsAddAction(Action))
			{
				OutResults.Add(DepotFileName, EPerforceState::OpenForAdd);
			}
			else if (Action == TEXT("edit"))
			{
				OutResults.Add(DepotFileName, EPerforceState::CheckedOut);
			}
			else if (IsDeleteAction(Action))
			{
				OutResults.Add(DepotFileName, EPerforceState::MarkedForDelete);
			}
			else if (Action == TEXT("branch") || Action == TEXT("integrate"))
			{
				OutResults.Add(DepotFileName, EPerforceState::Branched);
			}
		}
	}
}

static void ParseShelvedChangelistResults(const FP4RecordSet& InRecords, TMap<FString, EPerforceState::Type>& OutResults)
{
	// Describe returns only one record.
	check(InRecords.Num() == 1);
	const FP4Record& Record = InRecords[0];

	for (int32 FileIndex = 0;; ++FileIndex)
	{
		const FString& DepotFileName = Record(FString::Printf(TEXT("depotFile%d"), FileIndex));
		const FString& Action = Record(FString::Printf(TEXT("action%d"), FileIndex));

		if (DepotFileName.Len() == 0)
			break;

		if (Action.Len() > 0)
		{
			if (IsAddAction(Action))
			{
				OutResults.Add(DepotFileName, EPerforceState::OpenForAdd);
			}
			else if (Action == TEXT("edit"))
			{
				OutResults.Add(DepotFileName, EPerforceState::CheckedOut);
			}
			else if (IsDeleteAction(Action))
			{
				OutResults.Add(DepotFileName, EPerforceState::MarkedForDelete);
			}
			else if (Action == TEXT("branch") || Action == TEXT("integrate"))
			{
				OutResults.Add(DepotFileName, EPerforceState::Branched);
			}
		}
	}
}

static const FString& FindWorkspaceFile(const TArray<FPerforceSourceControlState>& InStates, const FString& InDepotFile)
{
	for(auto It(InStates.CreateConstIterator()); It; It++)
	{
		if(It->DepotFilename == InDepotFile)
		{
			return It->LocalFilename;
		}
	}

	return InDepotFile;
}

static void ParseHistoryResults(FPerforceSourceControlProvider& SCCProvider, const FP4RecordSet& InRecords, const TArray<FPerforceSourceControlState>& InStates, FPerforceFileHistoryMap& OutHistory)
{
	if (InRecords.Num() > 0)
	{
		// Iterate over each record, extracting the relevant information for each
		for (const FP4Record& ClientRecord : InRecords)
		{
			// Extract the file name
			check(ClientRecord.Contains(TEXT("depotFile")));
			const FString& DepotFileName = ClientRecord(TEXT("depotFile"));
			const FString& LocalFileName = FindWorkspaceFile(InStates, DepotFileName);

			if (OutHistory.Contains(LocalFileName))
			{
				// If a file existed in a location multiple times we rely on the first, most recent record having all the revisions we are interested in
				continue;
			}

			TArray< TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> > Revisions;
			int32 RevisionNumbers = 0;
			for (;;)
			{
				// Extract the revision number
				FString VarName = FString::Printf(TEXT("rev%d"), RevisionNumbers);
				if (!ClientRecord.Contains(*VarName))
				{
					// No more revisions
					break;
				}
				const FString& RevisionNumber = ClientRecord(*VarName);

				// Extract the user name
				VarName = FString::Printf(TEXT("user%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& UserName = ClientRecord(*VarName);

				// Extract the date
				VarName = FString::Printf(TEXT("time%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& Date = ClientRecord(*VarName);

				// Extract the changelist number
				VarName = FString::Printf(TEXT("change%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& ChangelistNumber = ClientRecord(*VarName);

				// Extract the description
				VarName = FString::Printf(TEXT("desc%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& Description = ClientRecord(*VarName);

				// Extract the action
				VarName = FString::Printf(TEXT("action%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& Action = ClientRecord(*VarName);

				FString FileSize(TEXT("0"));

				// Extract the file size
				if(!IsDeleteAction(Action)) //delete actions don't have a fileSize from PV4
				{
					VarName = FString::Printf(TEXT("fileSize%d"), RevisionNumbers);
					check(ClientRecord.Contains(*VarName));
					FileSize = ClientRecord(*VarName);
				}

				// Extract the clientspec/workspace
				VarName = FString::Printf(TEXT("client%d"), RevisionNumbers);
				check(ClientRecord.Contains(*VarName));
				const FString& ClientSpec = ClientRecord(*VarName);

				// check for branch
				TSharedPtr<FPerforceSourceControlRevision, ESPMode::ThreadSafe> BranchSource;
				VarName = FString::Printf(TEXT("how%d,0"), RevisionNumbers);
				if(ClientRecord.Contains(*VarName))
				{
					BranchSource = MakeShareable(new FPerforceSourceControlRevision(SCCProvider));

					VarName = FString::Printf(TEXT("file%d,0"), RevisionNumbers);
					const FString& BranchSourceFileName = ClientRecord(*VarName);
					BranchSource->FileName = FindWorkspaceFile(InStates, BranchSourceFileName);

					VarName = FString::Printf(TEXT("erev%d,0"), RevisionNumbers);
					const FString& BranchSourceRevision = ClientRecord(*VarName);
					BranchSource->RevisionNumber = FCString::Atoi(*BranchSourceRevision);
				}

				TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> Revision = MakeShareable(new FPerforceSourceControlRevision(SCCProvider));
				Revision->FileName = LocalFileName;
				Revision->RevisionNumber = FCString::Atoi(*RevisionNumber);
				Revision->Revision = RevisionNumber;
				Revision->ChangelistNumber = FCString::Atoi(*ChangelistNumber);
				Revision->Description = Description;
				Revision->UserName = UserName;
				Revision->ClientSpec = ClientSpec;
				Revision->Action = Action;
				Revision->BranchSource = BranchSource;
				Revision->Date = FDateTime(1970, 1, 1, 0, 0, 0, 0) + FTimespan::FromSeconds(FCString::Atoi(*Date));
				Revision->FileSize = FCString::Atoi(*FileSize);

				Revisions.Add(Revision);

				RevisionNumbers++;
			}

			if(Revisions.Num() > 0)
			{
				OutHistory.Add(LocalFileName, Revisions);
			}
		}
	}
}

static bool GetFileHistory(	FPerforceSourceControlProvider& SCCProvider, FPerforceConnection& Connection, FPerforceSourceControlCommand& InCommand, 
							const TArray<FString>& InFiles, TArray<FPerforceSourceControlState>& OutStates, FPerforceFileHistoryMap& OutHistory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::GetFileHistory);

	TArray<FString> Parameters;
	FP4RecordSet Records;
	// disregard non-contributory integrations
	Parameters.Add(TEXT("-s"));
	//include branching history
	Parameters.Add(TEXT("-i"));
	//include truncated change list descriptions
	Parameters.Add(TEXT("-L"));
	//include time stamps
	Parameters.Add(TEXT("-t"));
	//limit to last 100 changes
	Parameters.Add(TEXT("-m 100"));
	Parameters.Append(InFiles);
	InCommand.bCommandSuccessful &= Connection.RunCommand(TEXT("filelog"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
	ParseHistoryResults(SCCProvider, Records, OutStates, OutHistory);
	RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."));
	RemoveRedundantErrors(InCommand, TEXT(" - file(s) not on client"));
	RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"));

	return InCommand.bCommandSuccessful;
}

static void ParseDiffResults(const FP4RecordSet& InRecords, TArray<FString>& OutModifiedFiles)
{
	if (InRecords.Num() > 0)
	{
		// Iterate over each record found as a result of the command, parsing it for relevant information
		for (const FP4Record& ClientRecord : InRecords)
		{
			FString FileName = ClientRecord(TEXT("clientFile"));
			FPaths::NormalizeFilename(FileName);
			OutModifiedFiles.Add(FileName);
		}
	}
}

static void ParseChangelistsResults(const FP4RecordSet& InRecords, TArray<FPerforceSourceControlChangelistState>& OutStates)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& ChangelistString = ClientRecord(TEXT("change"));
		int32 ChangelistNumber = FCString::Atoi(*ChangelistString);

		FPerforceSourceControlChangelist Changelist(ChangelistNumber);

		FPerforceSourceControlChangelistState& State = OutStates.Emplace_GetRef(Changelist);
		State.Description = ClientRecord(TEXT("desc"));
		State.bHasShelvedFiles = ClientRecord.Contains(TEXT("shelved"));
	}
}

FName FPerforceUpdateStatusWorker::GetName() const
{
	return "UpdateStatus";
}

bool FPerforceUpdateStatusWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceUpdateStatusWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FUpdateStatus>(InCommand.Operation);

		if(InCommand.Files.Num())
		{
			// See http://www.perforce.com/perforce/doc.current/manuals/cmdref/p4_fstat.html
			// for full reference info on fstat command parameters...

			TArray<FString> Parameters;

			// We want to include integration record information:
			Parameters.Add(TEXT("-Or"));

			// Get the branches of interest for status updates
			const TArray<FString>& ContentRoots = InCommand.ContentRoots;
			const TArray<FString>& StatusBranches = InCommand.StatusBranchNames;

			// Mandatory parameters (the list of files to stat):
			for (FString File : InCommand.Files)
			{
				if (Operation->IsDirectoryPath(File))
				{
					// If the file is a directory, do a recursive fstat on the contents
					File /= TEXT("...");
				}
				else
				{
					for (const FString& Branch : StatusBranches)
					{
						for (const FString& ContentRoot : ContentRoots)
						{
							// Check the status branch for updates
							FString BranchFile;
							if (File.Split(ContentRoot, nullptr, &BranchFile))
							{
								// Ignore collection files when querying status branches
								FString Ext = FPaths::GetExtension(BranchFile, true);
								if (Ext.Compare(TEXT(".collection"), ESearchCase::IgnoreCase) == 0)
								{
									continue;
								}

								TArray<FStringFormatArg> Args = { Branch, ContentRoot, BranchFile };
								Parameters.Add(FString::Format(TEXT("{0}/{1}{2}"), Args));
							}
						}
					}
				}

				Parameters.Add(MoveTemp(File));
			}

			// Initially successful
			InCommand.bCommandSuccessful = true;

			ERunCommandFlags Flags = ERunCommandFlags::Default;
#if !NO_LOGGING
			// FPerforceConnect::RunCommand can log a lot of spam when it comes to fstat which is determintal to editor performance.
			// So we want to suppress that log output unless the user is running with LogSourceControl as verbose.
			if (LogSourceControl.IsSuppressed(ELogVerbosity::Verbose))
			{
				Flags |= ERunCommandFlags::DisableCommandLogging;
			}
#endif // !NO_LOGGING

			// Parse branch modifications
			TMap<FString, FBranchModification> BranchModifications;
			if (StatusBranches.Num())
			{
				// Get all revisions to check for modifications on other branches
				TArray<FString> RevisionParameters = Parameters;
				// Sort by head revision
				RevisionParameters.Insert(TEXT("-Sr"), 0);
				// Note: -Of suppresses open[...], so must be generated in a separate query
				RevisionParameters.Insert(TEXT("-Of"), 0);

				FP4RecordSet RevisionRecords;
				InCommand.bCommandSuccessful &= Connection.RunCommand(TEXT("fstat"), RevisionParameters, RevisionRecords, nullptr, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped, Flags);
				ParseBranchModificationResults(RevisionRecords, InCommand.ResultInfo.ErrorMessages, ContentRoots, BranchModifications);
			}

			FP4RecordSet Records;
			InCommand.bCommandSuccessful &= Connection.RunCommand(TEXT("fstat"), Parameters, Records, nullptr, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped, Flags);
			ParseUpdateStatusResults(Records, InCommand.ResultInfo.ErrorMessages, OutStates, ContentRoots, BranchModifications);
			RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."), false);
			RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"));
			RemoveRedundantErrors(InCommand, TEXT(" - file(s) not in client view"), false);
			RemoveRedundantErrors(InCommand, TEXT(" - protected namespace - access denied"), false);

			// Parse ignores for files that landed in NotInDepot as 'fstat' output does not differentiate between NotInDepot vs Ignore.
			if (GParseIgnores)
			{
				for (FPerforceSourceControlState& State : OutStates)
				{
					if (State.State == EPerforceState::NotInDepot)
					{
						const bool bIsIgnored = Connection.IsIgnored(State.GetFilename());
						if (bIsIgnored)
						{
							State.State = EPerforceState::Ignore;
						}
					}
				}
			}
		}
		else
		{
			InCommand.bCommandSuccessful = true;
		}

		if(Operation->ShouldUpdateHistory())
		{
			GetFileHistory(GetSCCProvider(), Connection, InCommand, InCommand.Files, OutStates, OutHistory);
		}

		if(Operation->ShouldGetOpenedOnly())
		{
			const FString ContentFolder = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
			const FString FileQuery = FString::Printf(TEXT("%s..."), *ContentFolder);
			TArray<FString> Parameters = InCommand.Files;
			Parameters.Add(FileQuery);
			FP4RecordSet Records;
			Connection.RunCommand(TEXT("opened"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
			InCommand.bCommandSuccessful &= (InCommand.ResultInfo.ErrorMessages.Num() == 0);
			ParseOpenedResults(Records, ANSI_TO_TCHAR(Connection.P4Client.GetClient().Text()), Connection.ClientRoot, OutStateMap);
			RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."));
			RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"));
		}

		if(Operation->ShouldUpdateModifiedState())
		{
			TArray<FString> Parameters;
			FP4RecordSet Records;
			// Query for open files different than the versions stored in Perforce

			if (Operation->ShouldForceUpdate())
			{
				Parameters.Add(TEXT("-f"));
			}

			Parameters.Add(TEXT("-sa"));
			for (FString File : InCommand.Files)
			{
				if (IFileManager::Get().DirectoryExists(*File))
				{
					// If the file is a directory, do a recursive diff on the contents
					File /= TEXT("...");
				}
								
				if (Operation->ShouldUpdateModifiedStateToLocalRevision())
				{
					File += TEXT("#have");
				}

				Parameters.Add(MoveTemp(File));
			}
			InCommand.bCommandSuccessful &= Connection.RunCommand(TEXT("diff"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

			bool bMoveErrorsToInfo = !Operation->ShouldBeQuiet();

			// Parse the results and store them in the command
			ParseDiffResults(Records, OutModifiedFiles);
			RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."));
			RemoveRedundantErrors(InCommand, TEXT(" - file(s) not opened for edit"));
			RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"));
			RemoveRedundantErrors(InCommand, TEXT(" - file(s) not opened on this client"));
			RemoveRedundantErrors(InCommand, TEXT("- file(s) not on client."), bMoveErrorsToInfo);
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceUpdateStatusWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceUpdateStatusWorker::UpdateStates);

	bool bUpdated = false;

	const FDateTime Now = FDateTime::Now();

	// first update cached state from 'fstat' call
	for(int StatusIndex = 0; StatusIndex < OutStates.Num(); StatusIndex++)
	{
		const FPerforceSourceControlState& Status = OutStates[StatusIndex];
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = GetSCCProvider().GetStateInternal(Status.LocalFilename);
		// Update every member except History and Timestamp. History will be updated below from the OutHistory map.
		// Timestamp is used to throttle status requests, so update it to current time:
		auto History = MoveTemp(State->History);
		*State = Status;
		State->History = MoveTemp(History);
		State->TimeStamp = Now;
		bUpdated = true;
		PerforceWorkersTedsUtility::UpdateEditorDataStorage(*State);
	}

	// next update state from 'opened' call
	bUpdated |= UpdateCachedStates(GetSCCProvider(), OutStateMap);

	// add history, if any
	for(FPerforceFileHistoryMap::TConstIterator It(OutHistory); It; ++It)
	{
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = GetSCCProvider().GetStateInternal(It.Key());
		const TArray< TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> >& History = It.Value();
		State->History = History;
		State->TimeStamp = Now;
		bUpdated = true;
		PerforceWorkersTedsUtility::UpdateEditorDataStorage(*State);
	}

	// add modified state
	for(int ModifiedIndex = 0; ModifiedIndex < OutModifiedFiles.Num(); ModifiedIndex++)
	{
		const FString& FileName = OutModifiedFiles[ModifiedIndex];
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> State = GetSCCProvider().GetStateInternal(FileName);
		State->bModifed = true;
		State->TimeStamp = Now;
		bUpdated = true;
		PerforceWorkersTedsUtility::UpdateEditorDataStorage(*State);
	}

	return !bForceQuiet && bUpdated;
}

FName FPerforceGetWorkspacesWorker::GetName() const
{
	return "GetWorkspaces";
}

bool FPerforceGetWorkspacesWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetWorkspacesWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FScWorkspaceInfo> OutWorkspaces;
		InCommand.bCommandSuccessful = Connection.GetWorkspaces(InCommand.ConnectionInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), OutWorkspaces, InCommand.ResultInfo);

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FGetWorkspaces, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetWorkspaces>(InCommand.Operation);
		Operation->Results = MoveTemp(OutWorkspaces);
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceGetWorkspacesWorker::UpdateStates() const
{
	return false;
}

FName FPerforceGetProjectWorkspacesWorker::GetName() const
{
	return "GetProjectWorkspaces";
}

bool FPerforceGetProjectWorkspacesWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetProjectWorkspacesWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> ClientSpecList;
		InCommand.bCommandSuccessful = Connection.GetProjectWorkspaces(InCommand.ConnectionInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), ClientSpecList, InCommand.ResultInfo);

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FGetProjectWorkspaces, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetProjectWorkspaces>(InCommand.Operation);
		Operation->Results = ClientSpecList;
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceGetProjectWorkspacesWorker::UpdateStates() const
{
	return false;
}

FName FPerforceGetPendingChangelistsWorker::GetName() const
{
	return "GetPendingChangelists";
}

static bool GetOpenedFilesInChangelist(
	FPerforceConnection& Connection,
	FPerforceSourceControlCommand& InCommand,
	TOptional<FPerforceSourceControlChangelist> Changelist,
	TArray<FPerforceSourceControlState>& FilesStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::GetOpenedFilesInChangelist);

	TArray<FString> Parameters;
	if (Changelist.IsSet())
	{
		Parameters.Add(TEXT("-c"));	// -c	Changelist
		Parameters.Add(Changelist.GetValue().ToString());	// <changelist>
	}
	else 
	{
		Parameters.Add(TEXT("-u"));									// -u			For user
		Parameters.Add(InCommand.ConnectionInfo.UserName);			// <username>
		Parameters.Add(TEXT("-C"));									// -C			For workspace
		Parameters.Add(InCommand.ConnectionInfo.Workspace);			// <workspace>
	}

	FP4RecordSet Records;
	Connection.RunCommand(TEXT("opened"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
	InCommand.bCommandSuccessful &= (InCommand.ResultInfo.ErrorMessages.Num() == 0);

	if (InCommand.bCommandSuccessful)
	{
		ParseOpenedResults(Records, ANSI_TO_TCHAR(Connection.P4Client.GetClient().Text()), Connection.ClientRoot, FilesStates);
	}

	return InCommand.bCommandSuccessful;
}

static void ParseWhereResults(FP4RecordSet& InRecords, TMap<FString, FString>& DepotToFileMap)
{
	for (const FP4Record& Record : InRecords)
	{
		const FString& DepotFile = Record(TEXT("depotFile"));
		FString ClientFile = Record(TEXT("path")).Replace(TEXT("\\"), TEXT("/"));

		if (DepotFile.Len() > 0 && ClientFile.Len() > 0)
		{
			DepotToFileMap.Emplace(DepotFile, ClientFile);
		}
	}
}

static bool GetDepotFileToLocalFileMap(FPerforceConnection& Connection, FPerforceSourceControlCommand& InCommand, const TMap<FString, EPerforceState::Type>& InDepotFiles, TMap<FString, FString>& OutDepotToLocalMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::GetDepotFileToLocalFileMap);

	if (InDepotFiles.Num() == 0)
	{
		return true;
	}

	TArray<FString> Parameters;
	for (TMap<FString, EPerforceState::Type>::TConstIterator It(InDepotFiles); It; ++It)
	{
		Parameters.Add(It.Key());
	}

	FP4RecordSet Records;
	Connection.RunCommand(TEXT("where"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

	if (!InCommand.ResultInfo.HasErrors())
	{
		ParseWhereResults(Records, OutDepotToLocalMap);
		return true;
	}
	else
	{
		return false;
	}
}

bool FPerforceGetPendingChangelistsWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetPendingChangelistsWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FUpdatePendingChangelistsStatus>(InCommand.Operation);

		InCommand.bCommandSuccessful = true;

		if (Operation->ShouldUpdateAllChangelists())
		{
			// First, insert the default changelist which always exists
			FPerforceSourceControlChangelistState& State = OutChangelistsStates.Emplace_GetRef(FPerforceSourceControlChangelist::DefaultChangelist);
		}

		{
			// Query all pending change list (the default is not listed by this request) for this user/workspace. (When a specific set of changelists is requested,
			// we could use -e to restrict the returned list, but it returns CL greater or equal to the CL specified. In general, we likely not gain much compare to list all pending ones.)
			TArray<FString> Parameters;
			Parameters.Add(TEXT("-l"));									// -l			Complete description
			Parameters.Add(TEXT("-spending"));							// -s pending	Only pending changelists
			Parameters.Add(TEXT("-u"));									// -u			For user
			Parameters.Add(InCommand.ConnectionInfo.UserName);			// <username>
			Parameters.Add(TEXT("-c"));									// -c			For workspace
			Parameters.Add(InCommand.ConnectionInfo.Workspace);			// <workspace>

			FP4RecordSet Records;
			Connection.RunCommand(TEXT("changes"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
			InCommand.bCommandSuccessful &= (InCommand.ResultInfo.ErrorMessages.Num() == 0);

			ParseChangelistsResults(Records, OutChangelistsStates);

			// Remove the changelist that were not requested by the user.
			if (!Operation->ShouldUpdateAllChangelists())
			{
				const TArray<FSourceControlChangelistRef>& RequestedChangelists = Operation->GetChangelistsToUpdate();
				OutChangelistsStates.RemoveAll([&RequestedChangelists](const FPerforceSourceControlChangelistState& ChangelistState)
				{
					FPerforceSourceControlChangelistRef RemoveChangelistCandidate = StaticCastSharedRef<FPerforceSourceControlChangelist>(ChangelistState.GetChangelist());
					return !RequestedChangelists.ContainsByPredicate([&RemoveChangelistCandidate](const FSourceControlChangelistRef& Requested)
					{
						return StaticCastSharedRef<FPerforceSourceControlChangelist>(Requested)->ToInt() == RemoveChangelistCandidate->ToInt();
					});
				});
			}
		}

		bCleanupCache = InCommand.bCommandSuccessful;

		// Test whether we should continue processing SCC commands
		auto ShouldContinueProcessing = [&InCommand]() { return InCommand.bCommandSuccessful && !InCommand.IsCanceled(); };

		if (Operation->ShouldUpdateFilesStates())
		{
			OutCLFilesStates.Reserve(OutChangelistsStates.Num());
			TArray<FPerforceSourceControlState> AllFilesStates;
			GetOpenedFilesInChangelist(Connection, InCommand, {}, AllFilesStates);

			if (ShouldContinueProcessing() && AllFilesStates.Num() && OutChangelistsStates.Num())
			{
				Algo::SortBy(AllFilesStates, [](const FPerforceSourceControlState& State){ return State.Changelist.ToInt(); });
				Algo::SortBy(OutChangelistsStates, [](const FPerforceSourceControlChangelistState& State){ 
					return StaticCastSharedRef<FPerforceSourceControlChangelist>(State.GetChangelist())->ToInt();});
				
				// OutChangelistStates is the authoritative list of changelists we want to provide results for, any file states which don't have a matching changelist can be discarded
				OutCLFilesStates.AddDefaulted(OutChangelistsStates.Num());
				for (int32 ChangelistIndex = 0, FileIndex = 0; FileIndex < AllFilesStates.Num() && ChangelistIndex < OutChangelistsStates.Num(); ++ChangelistIndex)
				{
					const int32 ChangelistChangelist = StaticCastSharedRef<FPerforceSourceControlChangelist>(OutChangelistsStates[ChangelistIndex].GetChangelist())->ToInt();
					for (; FileIndex < AllFilesStates.Num(); ++FileIndex)
					{
						int32 FileChangelist = AllFilesStates[FileIndex].Changelist.ToInt();
						if(FileChangelist == ChangelistChangelist)
						{
							OutCLFilesStates[ChangelistIndex].Add(MoveTemp(AllFilesStates[FileIndex]));
						}
						else if (FileChangelist < ChangelistChangelist)
						{
							// Skip files in skipped changelists until we find a file for this changelist or a later one
							continue;
						}
						else 
						{
							break;
						}
					}
				}
			}
		}

		if(Operation->ShouldUpdateShelvedFilesStates())
		{
			OutCLShelvedFilesStates.Reserve(OutChangelistsStates.Num());

			for (FPerforceSourceControlChangelistState& ChangelistState : OutChangelistsStates)
			{
				if (!ShouldContinueProcessing())
				{
					break;
				}

				if (!ChangelistState.bHasShelvedFiles)
				{
					OutCLShelvedFilesStates.Emplace(); // Add empty list
					OutCLShelvedFilesMap.Emplace(); // Add empty list
					continue;
				}

				TArray<FString> Parameters;
				Parameters.Add(TEXT("-s"));
				Parameters.Add(TEXT("-S"));
				Parameters.Add(ChangelistState.Changelist.ToString());

				FP4RecordSet Records;
				InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("describe"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

				if (InCommand.bCommandSuccessful)
				{
					TMap<FString, EPerforceState::Type>& OutShelvedStateMap = OutCLShelvedFilesStates.Emplace_GetRef();
					ParseShelvedChangelistResults(Records, OutShelvedStateMap);

					TMap<FString, FString>& OutShelvedFileMap = OutCLShelvedFilesMap.Emplace_GetRef();
					GetDepotFileToLocalFileMap(Connection, InCommand, OutShelvedStateMap, OutShelvedFileMap);

					// Remove errors encountered in GetDepotFileToLocalFileMap when parsing changelists in other workspaces
					// as we expect those files to not be in the client view
					// TODO: The root cause of this issue is we iterate over all changelists but in general we only care
					// about the changelists for the active workspace, we should
					// a) Not try to resolve local files for changelists in other workspaces
					// b) Provide better filtering so the user can request this operation only run on changelists for the
					// active workspace.
					const bool bMoveToInfo = false;
					RemoveRedundantErrors(InCommand, TEXT(" - file(s) not in client view."), bMoveToInfo);
				}
			}
		}
	}

	if (InCommand.IsCanceled() || !InCommand.bCommandSuccessful)
	{
		OutChangelistsStates.Empty();
		OutCLFilesStates.Empty();
		OutCLShelvedFilesStates.Empty();
	}

	return InCommand.bCommandSuccessful;
}

static bool AddShelvedFilesToChangelist(FPerforceSourceControlProvider& SCCProvider, const TMap<FString, EPerforceState::Type>& FilesToAdd, 
										const TMap<FString, FString>& DepotToFileMap, TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe>& ChangelistState, 
										const FDateTime* TimeStamp = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforce::AddShelvedFilesToChangelist);

	const FDateTime Now = (TimeStamp ? *TimeStamp : FDateTime::Now());

	for (TMap<FString, EPerforceState::Type>::TConstIterator It(FilesToAdd); It; ++It)
	{
		FString ItDepotFilename = It.Key();
		FString ItFilename = ItDepotFilename;

		if (DepotToFileMap.Contains(ItDepotFilename))
		{
			ItFilename = DepotToFileMap[It.Key()];
		}

		FSourceControlStateRef* SourceControlStateRef = ChangelistState->ShelvedFiles.Find(ItFilename);

		if (!SourceControlStateRef)
		{
			// Create new entry
			TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> ShelvedFileState = MakeShareable(new FPerforceSourceControlState(ItFilename));
			ShelvedFileState->DepotFilename = ItDepotFilename;

			// Add revision to be able to fetch the shelved file, if it's not marked for deletion.
			if (It.Value() != EPerforceState::MarkedForDelete)
			{
				TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> ShelvedRevision = MakeShareable(new FPerforceSourceControlRevision(SCCProvider));
				ShelvedRevision->FileName = ShelvedFileState->DepotFilename;
				ShelvedRevision->ChangelistNumber = StaticCastSharedRef<FPerforceSourceControlChangelist>(ChangelistState->GetChangelist())->ToInt();
				ShelvedRevision->bIsShelve = true;

				ShelvedFileState->History.Add(ShelvedRevision);
			}

			// Add to shelved files
			SourceControlStateRef = &ChangelistState->ShelvedFiles.Add(ItFilename, ShelvedFileState);
		}

		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> FileState = StaticCastSharedRef<FPerforceSourceControlState>(*SourceControlStateRef);

 		FileState->SetState(It.Value());
 		FileState->TimeStamp = Now;
	}

	return FilesToAdd.Num() > 0;
}

static bool AddShelvedFilesToChangelist(FPerforceSourceControlProvider& SCCProvider, const TMap<FString, EPerforceState::Type>& FilesToAdd, 
										const TMap<FString, FString>& DepotToFileMap, const FPerforceSourceControlChangelist& Changelist)
{
	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = SCCProvider.GetStateInternal(Changelist);
	return AddShelvedFilesToChangelist(SCCProvider, FilesToAdd, DepotToFileMap, ChangelistState);
}

bool FPerforceGetPendingChangelistsWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetPendingChangelistsWorker::UpdateStates);

	bool bUpdated = false;

	const FDateTime Now = FDateTime::Now();

	// first update cached state from 'changes' call
	for (int StatusIndex = 0; StatusIndex < OutChangelistsStates.Num(); StatusIndex++)
	{
		const FPerforceSourceControlChangelistState& CLStatus = OutChangelistsStates[StatusIndex];
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetSCCProvider().GetStateInternal(CLStatus.Changelist);
		// Timestamp is used to throttle status requests, so update it to current time:
		*ChangelistState = CLStatus;
		ChangelistState->TimeStamp = Now;
		bUpdated = true;

		// Update files states for files in the changelist
		bool bUpdateFilesStates = (OutCLFilesStates.Num() == OutChangelistsStates.Num());
		if (bUpdateFilesStates)
		{
			ChangelistState->Files.Reset(OutCLFilesStates[StatusIndex].Num());
			for (const auto& FileState : OutCLFilesStates[StatusIndex])
			{
				TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> CachedFileState = GetSCCProvider().GetStateInternal(FileState.LocalFilename);
				CachedFileState->Update(FileState, &Now);
				ChangelistState->Files.AddUnique(CachedFileState);
			}
		}

		// Update shelved files in the the changelist
		bool bUpdateShelvedFiles = (OutCLShelvedFilesStates.Num() == OutChangelistsStates.Num());
		if(bUpdateShelvedFiles)
		{
			ChangelistState->ShelvedFiles.Reset();
			ChangelistState->ShelvedFiles.Reserve(OutCLShelvedFilesStates[StatusIndex].Num());
			AddShelvedFilesToChangelist(GetSCCProvider(), OutCLShelvedFilesStates[StatusIndex], OutCLShelvedFilesMap[StatusIndex], ChangelistState, &Now);
		}
	}

	if (bCleanupCache)
	{
		TArray<FPerforceSourceControlChangelist> ChangelistsToRemove;
		GetSCCProvider().GetCachedStateByPredicate([this, &ChangelistsToRemove](const FSourceControlChangelistStateRef& InCLState) {
			TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> CLState = StaticCastSharedRef<FPerforceSourceControlChangelistState>(InCLState);

			if (Algo::NoneOf(OutChangelistsStates, [&CLState](const FPerforceSourceControlChangelistState& UpdatedCLState) {
				return CLState->Changelist == UpdatedCLState.Changelist;
				}))
			{
				ChangelistsToRemove.Add(CLState->Changelist);
			}
			
			return false;
			});

		for (const FPerforceSourceControlChangelist& ChangelistToRemove : ChangelistsToRemove)
		{
			GetSCCProvider().RemoveChangelistFromCache(ChangelistToRemove);
		}
	}

	return bUpdated;
}


FName FPerforceCopyWorker::GetName() const
{
	return "Copy";
}

bool FPerforceCopyWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCopyWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FCopy, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCopy>(InCommand.Operation);

		FString DestinationPath = FPaths::ConvertRelativePathToFull(Operation->GetDestination());

		TArray<FString> Parameters;

		AppendChangelistParameter(GetSCCProvider(), Parameters);

		FP4RecordSet Records;
		if (Operation->CopyMethod == FCopy::ECopyMethod::Branch || GAlwaysBranchFilesOnCopy)
		{
			Parameters.Append(InCommand.Files);
			Parameters.Add(DestinationPath);

			InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("integrate"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

			// We now need to do a p4 resolve.
			// This is because when we copy a file in the Editor, we first make the copy on disk before attempting to branch. This causes a conflict in P4's eyes.
			// We must do this to prevent the asset registry from picking up what it thinks is a newly-added file (which would be created by the p4 integrate command)
			// and then the package system getting very confused about where to save the now-duplicated assets.
			if (InCommand.bCommandSuccessful)
			{
				TArray<FString> ResolveParameters;
				switch (Operation->ResolveMethod)
				{
					case FCopy::EResolveMethod::AcceptTarget:
						ResolveParameters.Add(TEXT("-ay"));	// 'accept yours' to leave the target unchanged
					break;
					case FCopy::EResolveMethod::AcceptSource:
						ResolveParameters.Add(TEXT("-at")); // 'accept theirs' to allow branch over deleted files
					break;
				}
				ResolveParameters.Add(MoveTemp(DestinationPath));
				InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("resolve"), ResolveParameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
			}
		}
		else
		{
			Parameters.Add(MoveTemp(DestinationPath));

			check(Operation->CopyMethod == FCopy::ECopyMethod::Add);

			InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("add"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		}
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceCopyWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCopyWorker::UpdateStates);
	return UpdateCachedStates(GetSCCProvider(), OutResults);
}

// IPerforceSourceControlWorker interface
FName FPerforceResolveWorker::GetName() const
{
	return "Resolve";
}

bool FPerforceResolveWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceResolveWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		TArray<FString> Parameters;

		Parameters.Add("-ay");
		Parameters.Append(InCommand.Files);
		AppendChangelistParameter(GetSCCProvider(), Parameters);

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("resolve"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		if( InCommand.bCommandSuccessful )
		{
			UpdatedFiles = InCommand.Files;
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceResolveWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceResolveWorker::UpdateStates);

	for( const auto& Filename : UpdatedFiles )
	{
		auto State = GetSCCProvider().GetStateInternal( Filename );
		State->LocalRevNumber = State->DepotRevNumber;
		State->PendingResolveInfo = {};
	}

	return UpdatedFiles.Num() > 0;
}

FName FPerforceChangeStatusWorker::GetName() const
{
	return "ChangeStatus";
}

bool FPerforceChangeStatusWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceChangeStatusWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		TArray<FString> Parameters;
		Parameters.Append(InCommand.Files);

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("cstat"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		if (InCommand.bCommandSuccessful)
		{
			TSharedRef<FPerforceSourceControlChangeStatusOperation, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPerforceSourceControlChangeStatusOperation>(InCommand.Operation);

			for (const FP4Record& Record : Records)
			{
				const FString Changelist = Record[TEXT("change")];
				const FString StatusText = Record[TEXT("status")];
				EChangelistStatus Status = EChangelistStatus::Have;
				if (StatusText == TEXT("need"))
				{
					Status = EChangelistStatus::Need;
				}
				else if (StatusText == TEXT("partial"))
				{
					Status = EChangelistStatus::Partial;
				}
				
				Operation->OutResults.Add({ Changelist, Status });
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceChangeStatusWorker::UpdateStates() const
{
	return true;
}

FPerforceNewChangelistWorker::FPerforceNewChangelistWorker(FPerforceSourceControlProvider& InSourceControlProvider)
	: IPerforceSourceControlWorker(InSourceControlProvider)
	, NewChangelistState(NewChangelist)
{

}

FName FPerforceNewChangelistWorker::GetName() const
{
	return "NewChangelist";
}

bool FPerforceNewChangelistWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceNewChangelistWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FNewChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FNewChangelist>(InCommand.Operation);

		int32 ChangeList = Connection.CreatePendingChangelist(Operation->GetDescription(), TArray<FString>(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);

		InCommand.bCommandSuccessful = (ChangeList > 0);
		
		// Successfully created new changelist
		if (ChangeList > 0)
		{
			NewChangelist = FPerforceSourceControlChangelist(ChangeList);
			NewChangelistState.Changelist = NewChangelist;
			NewChangelistState.Description = Operation->GetDescription().ToString();
			NewChangelistState.bHasShelvedFiles = false;

			Operation->SetNewChangelist(MakeShared<FPerforceSourceControlChangelist>(NewChangelist));

			if (InCommand.Files.Num() > 0)
			{
				// Move files to changelist
				if (!RunReopenCommand(InCommand, InCommand.Files, NewChangelist, &MovedFiles))
				{
					// Move failed; delete newly created changelist
					FP4RecordSet Records;
					TArray<FString> ChangeParams;
					ChangeParams.Add(TEXT("-d"));
					ChangeParams.Add(NewChangelist.ToString());
					Connection.RunCommand(TEXT("change"), ChangeParams, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

					InCommand.bCommandSuccessful = false;
				}
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceNewChangelistWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceNewChangelistWorker::UpdateStates);

	if (NewChangelist.IsInitialized())
	{
		const FDateTime Now = FDateTime::Now();

		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetSCCProvider().GetStateInternal(NewChangelist);
		*ChangelistState = NewChangelistState;
		ChangelistState->TimeStamp = Now;

		for (const FString& MovedFile : MovedFiles)
		{
			TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> FileState = GetSCCProvider().GetStateInternal(MovedFile);

			// 1- Remove these files from their previous changelist
			TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetSCCProvider().GetStateInternal(FileState->Changelist);
			PreviousChangelist->Files.Remove(FileState);

			// 2- Add to the new changelist
			ChangelistState->Files.Add(FileState);

			// 3- Update changelist in file state
			FileState->Changelist = NewChangelist;
			FileState->TimeStamp = Now;
		}

		return true;
	}
	else
	{
		return false;
	}	
}


FName FPerforceDeleteChangelistWorker::GetName() const
{
	return "DeleteChangelist";
}

bool FPerforceDeleteChangelistWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteChangelistWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	// Can't delete the default changelist
	if (InCommand.Changelist.IsDefault())
	{
		InCommand.bCommandSuccessful = false;
	}
	else if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FDeleteChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FDeleteChangelist>(InCommand.Operation);

		FP4RecordSet Records;
		TArray<FString> Params;
		Params.Add(TEXT("-d"));
		Params.Add(InCommand.Changelist.ToString());
		// Command will fail if changelist is not empty
		Connection.RunCommand(TEXT("change"), Params, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		// The normal parsing of the records here will show that it failed, but there's no record on a deleted changelist
		InCommand.bCommandSuccessful = (InCommand.ResultInfo.ErrorMessages.Num() == 0);
		
		// Keep track of changelist to update the cache
		if (InCommand.bCommandSuccessful)
		{
			DeletedChangelist = InCommand.Changelist;
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceDeleteChangelistWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteChangelistWorker::UpdateStates);

	if (!DeletedChangelist.IsDefault())
	{
		return GetSCCProvider().RemoveChangelistFromCache(DeletedChangelist);
	}
	else
	{
		return false;
	}
}


FName FPerforceEditChangelistWorker::GetName() const
{
	return "EditChangelist";
}

bool FPerforceEditChangelistWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceEditChangelistWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FEditChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FEditChangelist>(InCommand.Operation);

		int32 ChangelistNumber = -1;

		if (InCommand.Changelist.IsDefault())
		{
			ChangelistNumber = Connection.CreatePendingChangelist(Operation->GetDescription(), InCommand.Files, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
		}
		else
		{
			ChangelistNumber = Connection.EditPendingChangelist(Operation->GetDescription(), InCommand.Changelist.ToInt(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
		}

		InCommand.bCommandSuccessful = (ChangelistNumber == InCommand.Changelist.ToInt() || (ChangelistNumber >= 0 && InCommand.Changelist.IsDefault()));

		if (InCommand.bCommandSuccessful)
		{
			EditedChangelist = FPerforceSourceControlChangelist(ChangelistNumber);
			EditedDescription = Operation->GetDescription();
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceEditChangelistWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceEditChangelistWorker::UpdateStates);

	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> EditedChangelistState = GetSCCProvider().GetStateInternal(EditedChangelist);
	// TODO: update similar to NewChangelist when/if we support files in edit/new changelists.
	EditedChangelistState->Description = EditedDescription.ToString();
	EditedChangelistState->Changelist = EditedChangelist;
	EditedChangelistState->TimeStamp = FDateTime::Now();

	return true;
}

FName FPerforceRevertUnchangedWorker::GetName() const
{
	return "RevertUnchanged";
}

bool FPerforceRevertUnchangedWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceRevertUnchangedWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;

		Parameters.Add(TEXT("-a")); // revert unchanged only
		Parameters.Add(TEXT("-c"));
		Parameters.Add(InCommand.Changelist.ToString());

		if (InCommand.Files.Num() > 0)
		{
			Parameters.Append(InCommand.Files);
		}	

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("revert"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		ParseRecordSetForState(Records, OutResults);
		ChangelistToUpdate = InCommand.Changelist;
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceRevertUnchangedWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceRevertUnchangedWorker::UpdateStates);

	bool bUpdatedStates = UpdateCachedStates(GetSCCProvider(), OutResults);
	bool bUpdatedChangelistState = ChangelistToUpdate.IsInitialized() && RemoveFilesFromChangelist(OutResults, GetSCCProvider(), ChangelistToUpdate);
	return bUpdatedStates || bUpdatedChangelistState;
}

FName FPerforceReopenWorker::GetName() const
{
	return "Reopen";
}
	
bool FPerforceReopenWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceReopenWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		ReopenedFiles.Reset(InCommand.Files.Num());
		InCommand.bCommandSuccessful = RunReopenCommand(InCommand, InCommand.Files, InCommand.Changelist, &ReopenedFiles);
		DestinationChangelist = InCommand.Changelist;
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceReopenWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceReopenWorker::UpdateStates);

	const FDateTime Now = FDateTime::Now();
	
	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> DestinationChangelistState = GetSCCProvider().GetStateInternal(DestinationChangelist);

	// 3 things to do here:
	for (const FString& ReopenedFile : ReopenedFiles)
	{
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> FileState = GetSCCProvider().GetStateInternal(ReopenedFile);

		// 1- Remove these files from their previous changelist
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetSCCProvider().GetStateInternal(FileState->Changelist);
		PreviousChangelist->Files.Remove(FileState);

		// 2- Add to the new changelist
		DestinationChangelistState->Files.Add(FileState);

		// 3- Update changelist in file state
		FileState->Changelist = DestinationChangelist;
		FileState->TimeStamp = Now;
	}
	
	return ReopenedFiles.Num() > 0;
}


FName FPerforceShelveWorker::GetName() const
{
	return "Shelve";
}

bool FPerforceShelveWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceShelveWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FShelve, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FShelve>(InCommand.Operation);

		FPerforceSourceControlChangelist Changelist(InCommand.Changelist);

		InCommand.bCommandSuccessful = true;

		// If the command is issued on the default changelist, then we should create a new changelist,
		// move the files to the new changelist (reopen), then shelve the files
		if (InCommand.Changelist.IsDefault())
		{
			TArray<FString> FilesToShelve = InCommand.Files;

			// If the command has specified the default changelist but no files, then get all files from the default changelist
			if (FilesToShelve.Num() == 0 && InCommand.Changelist.IsInitialized())
			{
				TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> DefaultChangelistState = GetSCCProvider().GetStateInternal(InCommand.Changelist);
				Algo::Transform(DefaultChangelistState->Files, FilesToShelve, [](const auto& FileState) {
					return FileState->GetFilename();
					});
			}

			int32 NewChangeList = Connection.CreatePendingChangelist(Operation->GetDescription(), TArray<FString>(), FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.ResultInfo);
			if (NewChangeList > 0)
			{
				Changelist = FPerforceSourceControlChangelist(NewChangeList);
				InCommand.bCommandSuccessful = RunReopenCommand(InCommand, FilesToShelve, Changelist, &MovedFiles);
				ChangelistDescription = Operation->GetDescription().ToString();
			}
			else
			{
				InCommand.bCommandSuccessful = false;
			}
		}

		FP4RecordSet Records;

		if (InCommand.bCommandSuccessful)
		{
			TArray<FString> Parameters;
			Parameters.Add(TEXT("-c"));
			Parameters.Add(Changelist.ToString());
			Parameters.Add(TEXT("-f")); // force

			if (InCommand.Files.Num() > 0)
			{
				Parameters.Append(InCommand.Files);
			}

			InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("shelve"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		}

		if (InCommand.bCommandSuccessful)
		{
			InChangelistToUpdate = InCommand.Changelist;
			OutChangelistToUpdate = Changelist;

			ParseShelvedResults(Records, OutResults);

			// Build depot to file mapping
			GetDepotFileToLocalFileMap(Connection, InCommand, OutResults, OutFileMap);
		}
		else
		{
			// If we had to create a new changelist, move the files back to the default changelist
			// and delete the changelist
			if (Changelist != InCommand.Changelist)
			{
				if (MovedFiles.Num() > 0)
				{
					RunReopenCommand(InCommand, MovedFiles, InCommand.Changelist);
				}

				TArray<FString> ChangeParams;
				ChangeParams.Add(TEXT("-d"));
				ChangeParams.Add(Changelist.ToString());
				Connection.RunCommand(TEXT("change"), ChangeParams, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceShelveWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceShelveWorker::UpdateStates);

	bool bMovedFiles = false;

	// If we moved files to a new changelist, then we must make sure that the files are properly moved
	if (InChangelistToUpdate != OutChangelistToUpdate && MovedFiles.Num() > 0)
	{
		const FDateTime Now = FDateTime::Now();
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> SourceChangelistState = GetSCCProvider().GetStateInternal(InChangelistToUpdate);
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> DestinationChangelistState = GetSCCProvider().GetStateInternal(OutChangelistToUpdate);

		DestinationChangelistState->Changelist = OutChangelistToUpdate;
		DestinationChangelistState->Description = ChangelistDescription;
		DestinationChangelistState->bHasShelvedFiles = true;

		for (const FString& MovedFile : MovedFiles)
		{
			TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> FileState = GetSCCProvider().GetStateInternal(MovedFile);

			SourceChangelistState->Files.Remove(FileState);
			DestinationChangelistState->Files.Add(FileState);
			FileState->Changelist = OutChangelistToUpdate;
			FileState->TimeStamp = Now;
		}

		bMovedFiles = true;
	}

	const bool bAddedShelvedFilesToChangelist = (OutResults.Num() > 0 && AddShelvedFilesToChangelist(GetSCCProvider(), OutResults, OutFileMap, OutChangelistToUpdate));

	return bMovedFiles || bAddedShelvedFilesToChangelist;
}


FName FPerforceDeleteShelveWorker::GetName() const
{
	return "DeleteShelved";
}

bool FPerforceDeleteShelveWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteShelveWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;
		Parameters.Add(TEXT("-d")); // -d is delete
		Parameters.Add(TEXT("-c"));
		Parameters.Add(InCommand.Changelist.ToString());

		if (InCommand.Files.Num() > 0)
		{
			Parameters.Append(InCommand.Files);
		}

		FP4RecordSet Records;
		Connection.RunCommand(TEXT("shelve"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		InCommand.bCommandSuccessful = (InCommand.ResultInfo.ErrorMessages.Num() == 0);

		if (InCommand.bCommandSuccessful)
		{
			ChangelistToUpdate = InCommand.Changelist;
			FilesToRemove = InCommand.Files;
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceDeleteShelveWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteShelveWorker::UpdateStates);

	if (ChangelistToUpdate.IsInitialized())
	{
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetSCCProvider().GetStateInternal(ChangelistToUpdate);

		if (FilesToRemove.Num() > 0)
		{
			bool bRemovedFiles = false;
			for (const FString& FileToRemove : FilesToRemove)
			{
				bRemovedFiles |= ChangelistState->ShelvedFiles.Remove(FileToRemove) > 0;
			}
			return bRemovedFiles;
		}

		bool bHadShelvedFiles = (ChangelistState->ShelvedFiles.Num() > 0);
		ChangelistState->ShelvedFiles.Reset();
		return bHadShelvedFiles;
	}

	return false;
}


FName FPerforceUnshelveWorker::GetName() const
{
	return "Unshelve";
}

bool FPerforceUnshelveWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceUnshelveWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);
	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TArray<FString> Parameters;

		Parameters.Add(TEXT("-s")); // unshelve from source changelist
		Parameters.Add(InCommand.Changelist.ToString()); // current changelist
		Parameters.Add(TEXT("-f")); // force overwriting of writeable but unopened files
		Parameters.Add(TEXT("-c")); // unshelve to target changelist
		Parameters.Add(InCommand.Changelist.ToString()); // current changelist

		if (InCommand.Files.Num() > 0)
		{
			Parameters.Append(InCommand.Files);
		}

		FP4RecordSet Records;
		// Note: unshelve can succeed partially.
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("unshelve"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);
		
		if (InCommand.bCommandSuccessful && Records.Num() > 0)
		{
			// At this point, the records contain the list of files from the depot that were unshelved
			// however they contain only the depot file equivalency; considering that some files might not be in the cache yet,
			// it is simpler to do a full update of the changelist files.
			ChangelistToUpdate = InCommand.Changelist;
			GetOpenedFilesInChangelist(Connection, InCommand, ChangelistToUpdate, ChangelistFilesStates);
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceUnshelveWorker::UpdateStates() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceUnshelveWorker::UpdateStates);

	if (ChangelistToUpdate.IsInitialized() && ChangelistFilesStates.Num() > 0)
	{
		const FDateTime Now = FDateTime::Now();

		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetSCCProvider().GetStateInternal(ChangelistToUpdate);

		ChangelistState->Files.Reset(ChangelistFilesStates.Num());
		for (const auto& FileState : ChangelistFilesStates)
		{
			TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> CachedFileState = GetSCCProvider().GetStateInternal(FileState.LocalFilename);
			CachedFileState->Update(FileState, &Now);

			ChangelistState->Files.AddUnique(CachedFileState);
		}

		return true;
	}
	else
	{
		return false;
	}
}

FName FPerforceDownloadFileWorker::GetName() const
{
	return "DownloadFile";
}

bool FPerforceDownloadFileWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDownloadFileWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		
		// TODO: The cast doesn't seem to have any type safety
		TSharedRef<FDownloadFile, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FDownloadFile>(InCommand.Operation);

		TArray<FString> Parameters;
		FP4RecordSet Records;

		const FString TargetDirectory = Operation->GetTargetDirectory();

		if (TargetDirectory.IsEmpty())
		{
			// Download and store the files as blobs in memory that the caller can access as they wish
			Parameters.Add(TEXT("-q")); // Do not print the header, we only want the actual file contents
			Parameters.Append(InCommand.Files);

			TArray<FSharedBuffer> FilesData;
			const ERunCommandFlags Flags = Operation->ShouldLogToStdOutput() ? ERunCommandFlags::Default : ERunCommandFlags::DisableCommandLogging;

			InCommand.bCommandSuccessful = Connection.RunCommand(	TEXT("print"), Parameters, Records, &FilesData, InCommand.ResultInfo,
																	FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), 
																	InCommand.bConnectionDropped, Flags);
			
			const bool bMoveToInfo = true;
			RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."), bMoveToInfo);
			RemoveRedundantErrors(InCommand, TEXT(" - file(s) not on client"), bMoveToInfo);
			RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"), bMoveToInfo);
			RemoveRedundantErrors(InCommand, TEXT(" - no file(s) at that changelist number."), bMoveToInfo);
			ReplaceErrors(InCommand, TEXT(" - must refer to client"), TEXT(" - no such depot!"), bMoveToInfo);

			if (!InCommand.ResultInfo.ErrorMessages.IsEmpty() || (InCommand.Files.Num() != FilesData.Num()))
			{
				InCommand.bCommandSuccessful = false;
			}

			if (InCommand.bCommandSuccessful)
			{
				check(InCommand.Files.Num() == FilesData.Num());

				for (int32 Index = 0; Index < InCommand.Files.Num(); ++Index)
				{
					Operation->__Internal_AddFileData(InCommand.Files[Index], FilesData[Index]);
				}
			}
		}
		else
		{
			// Downloading the files directly to a target directory
			for (const FString& TargetFilePath : InCommand.Files)
			{
				// First we will stream to a temp file before moving it to the final location if the download was successful.
				// This will prevent us from overwriting files with partial data in case of an error.

				Parameters.Add(TEXT("-o"));
				const FString BaseFilename = FPaths::GetBaseFilename(TargetFilePath);
				const FString TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));
				Parameters.Add(TempFilePath);

				Parameters.Add(TEXT("-q")); // Do not print the header, we only want the actual file contents
				Parameters.Add(TargetFilePath);

				const ERunCommandFlags Flags = Operation->ShouldLogToStdOutput() ? ERunCommandFlags::Default : ERunCommandFlags::DisableCommandLogging;

				InCommand.bCommandSuccessful = Connection.RunCommand(	TEXT("print"), Parameters, Records, nullptr, InCommand.ResultInfo, 
																		FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), 
																		InCommand.bConnectionDropped, Flags);

				const bool bMoveToInfo = true;
				RemoveRedundantErrors(InCommand, TEXT(" - no such file(s)."), bMoveToInfo);
				RemoveRedundantErrors(InCommand, TEXT(" - file(s) not on client"), bMoveToInfo);
				RemoveRedundantErrors(InCommand, TEXT("' is not under client's root '"), bMoveToInfo);
				RemoveRedundantErrors(InCommand, TEXT(" - no file(s) at that changelist number."), bMoveToInfo);
				ReplaceErrors(InCommand, TEXT(" - must refer to client"), TEXT(" - no such depot!"), bMoveToInfo);

				if (InCommand.bCommandSuccessful)
				{
					// To avoid Move() logging errors, first test if file exists
					// This ensures we respect what's specified by the operation using ISourceControlOperation::ShouldLogErrors()
					if (IFileManager::Get().FileExists(*TempFilePath))
					{
						const FString FinalTargetPath = TargetDirectory / FPaths::GetCleanFilename(TargetFilePath);
						if (IFileManager::Get().Move(*FinalTargetPath, *TempFilePath) == false)
						{
							// Give error 
							const FString ErrorMsg = FString::Printf(TEXT("Failed to move '%s' to '%s'"), *TempFilePath, *FinalTargetPath);
							InCommand.ResultInfo.ErrorMessages.Add(FText::FromString(ErrorMsg));
							InCommand.bCommandSuccessful = false;
						}
					}
					else
					{
						// Give error 
						const FString ErrorMsg = FString::Printf(TEXT("Failed to download '%s' to '%s'"), *TargetFilePath, *TempFilePath);
						InCommand.ResultInfo.ErrorMessages.Add(FText::FromString(ErrorMsg));
						InCommand.bCommandSuccessful = false;
					}
				}

				// Keep the already allocated memory for future files to prevent reallocation
				Parameters.Reset();
				Records.Reset();
			}
		}
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceDownloadFileWorker::UpdateStates() const
{
	// Downloading a file from the server will never affect the cached
	// file states.
	return false;
}

FName FPerforceRevisionInfoWorker::GetName() const
{
	return "GetSourceControlRevisionInfo";
}

bool FPerforceRevisionInfoWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceRevisionInfoWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		
		// TODO: The cast doesn't seem to have any type safety
		TSharedRef<FGetSourceControlRevisionInfo, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetSourceControlRevisionInfo>(InCommand.Operation);

		TArray<FString> Parameters;
		FP4RecordSet Records;

		// grab info about the provided files
		for (const FString& TargetFilePath : InCommand.Files)
		{
			Parameters.Add(TEXT("-t"));
			
			Parameters.Add(TEXT("-m"));
			Parameters.Add(TEXT("1"));
			
			Parameters.Add(TargetFilePath);

			const ERunCommandFlags Flags = Operation->ShouldLogToStdOutput() ? ERunCommandFlags::Default : ERunCommandFlags::DisableCommandLogging;


			InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("changes"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

			if (InCommand.bCommandSuccessful)
			{
				Operation->InternalSetResults(
					Records[0]["change"],
					{FDateTime(1970, 1, 1, 0, 0, 0, 0) + FTimespan::FromSeconds(FCString::Atoi(*Records[0]["time"]))},
					Records[0]["user"]);
			}

			// Keep the already allocated memory for future files to prevent reallocation
			Parameters.Reset();
			Records.Reset();
		}
	}
	return InCommand.bCommandSuccessful;
}

bool FPerforceRevisionInfoWorker::UpdateStates() const
{
	// This operation will never affect the cached
	// file states.
	return false;
}

FName FPerforceCreateWorkspaceWorker::GetName() const
{
	return "CreateWorkspace";
}

bool FPerforceCreateWorkspaceWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceCreateWorkspaceWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		TSharedRef<FCreateWorkspace, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCreateWorkspace>(InCommand.Operation);
	
		// Early out if the operation set up is invalid
		// Note if we set Stream and View this will not result in a p4 error and the View setting will be ignored.
		// As a precaution we will consider this an error instead to avoid unintended consequences for the caller.
		if (!Operation->GetWorkspaceStream().IsEmpty() && !Operation->GetClientView().IsEmpty())
		{
			FText ErrorMsg = LOCTEXT("WorkspaceInvalidSetup", "Cannot create a workspace for a stream and apply a client view at the same time");

			InCommand.ResultInfo.ErrorMessages.Add(ErrorMsg);
			InCommand.bCommandSuccessful = false;
			
			return InCommand.bCommandSuccessful;
		}

		TStringBuilder<2048> ClientDesc;

		ClientDesc << TEXT("Client:\t") << Operation->GetWorkspaceName() << TEXT("\n\n");
		ClientDesc << TEXT("Owner:\t") << Connection.GetUser() << TEXT("\n\n");

		if (!Operation->GetWorkspaceDescription().IsEmpty())
		{
			ClientDesc << TEXT("Description:\n\t") << Operation->GetWorkspaceDescription() << TEXT("\n\n");
		}

		ClientDesc << TEXT("Root:\t") << Operation->GetWorkspaceRoot() << TEXT("\n\n");

		if (!Operation->GetWorkspaceStream().IsEmpty())
		{
			ClientDesc << TEXT("Stream:\t") << Operation->GetWorkspaceStream() << TEXT("\n\n");
		}

		if(!Operation->GetClientView().IsEmpty())
		{
			ClientDesc << TEXT("View:\t\n");

			for (const FCreateWorkspace::FClientViewMapping& Mapping : Operation->GetClientView())
			{
				ClientDesc << TEXT("\t") << Mapping.Key;
				ClientDesc << TEXT(" ");
				ClientDesc << Mapping.Value << TEXT("\n");
			}

			ClientDesc << TEXT("\n");
		}

		AddType(*Operation, ClientDesc);

		InCommand.bCommandSuccessful = Connection.CreateWorkspace(	ClientDesc,
																	FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled),
																	InCommand.ResultInfo);
	}
	
	return InCommand.bCommandSuccessful;
}

void FPerforceCreateWorkspaceWorker::AddType(const FCreateWorkspace& Operation, FStringBuilderBase& ClientDesc)
{
	ClientDesc << TEXT("Type:\t");

	switch (Operation.GetType())
	{
		case FCreateWorkspace::EType::Writeable:
			ClientDesc << TEXT("writeable");
			break;
		case FCreateWorkspace::EType::ReadOnly:
			ClientDesc << TEXT("readonly");
			break;
		case FCreateWorkspace::EType::Partitioned:
			ClientDesc << TEXT("partitioned");
			break;
		default:
			checkNoEntry();
	}

	ClientDesc << TEXT("\n");
}


bool FPerforceCreateWorkspaceWorker::UpdateStates() const
{
	return false;
}

FName FPerforceDeleteWorkspaceWorker::GetName() const
{
	return "DeleteWorkspace";
}

bool FPerforceDeleteWorkspaceWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceDeleteWorkspaceWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		TSharedRef<FDeleteWorkspace, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FDeleteWorkspace>(InCommand.Operation);

		TArray<FString> Parameters;
		Parameters.Add(TEXT("-d")); // Do not print the header, we only want the actual file contents
		Parameters.Add(Operation->GetWorkspaceName());

		FP4RecordSet Records;
		// p4 client command does not return tagged results, which means info will get printed to
		// stdout which we do not want, so we can use the Quiet flag which is the same as passing
		// in -q as a global-opt in the command.
		const ERunCommandFlags RunFlags = ERunCommandFlags::Quiet;

		Connection.RunCommand(	TEXT("client"), 
								Parameters, 
								Records, 
								nullptr,
								InCommand.ResultInfo, 
								FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), 
								InCommand.bConnectionDropped,
								RunFlags);

		// p4 client -d doesn't return any records, so the return value of RunCommand will always
		// be false, so check the error messages instead.
		InCommand.bCommandSuccessful = InCommand.ResultInfo.ErrorMessages.IsEmpty();
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceDeleteWorkspaceWorker::UpdateStates() const
{
	return false;
}

FName FPerforceGetChangelistDetailsWorker::GetName() const
{
	return "GetChangelistDetails";
}

bool FPerforceGetChangelistDetailsWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetChangelistDetailsWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TSharedRef<FGetChangelistDetails, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetChangelistDetails>(InCommand.Operation);

		TArray<FString> Parameters;
		Parameters.Add(TEXT("-s"));
		Parameters.Add(TEXT("-S"));
		Parameters.Add(Operation->GetChangelistNumber());

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("describe"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

		if (InCommand.bCommandSuccessful)
		{
			TArray<TMap<FString, FString>> ChangelistRecord;
			ChangelistRecord.Reserve(Records.Num());

			for (FP4Record& P4Record : Records)
			{
				TMap<FString, FString> Record;

				for (TPair<FString, FString>& P4RecordPair : P4Record)
				{
					Record.Add(MoveTemp(P4RecordPair));
				}

				ChangelistRecord.Add(MoveTemp(Record));
			}

			Operation->SetChangelistDetails(MoveTemp(ChangelistRecord));
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceGetChangelistDetailsWorker::UpdateStates() const
{
	return false;
}

FName FPerforceGetFileWorker::GetName() const
{
	return "GetFile";
}

bool FPerforceGetFileWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceGetChangelistDetailsWorker::FPerforceGetFileWorker);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (InCommand.IsCanceled() || !ScopedConnection.IsValid())
	{
		return false;
	}

	TSharedRef<FGetFile, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetFile>(InCommand.Operation);

	FPerforceSourceControlRevision Revision(GetSCCProvider());

	Revision.FileName = Operation->GetDepotFilePath();
	Revision.ChangelistNumber = FCString::Atoi(*Operation->GetChangelistNumber());
	Revision.RevisionNumber = FCString::Atoi(*Operation->GetRevisionNumber());
	Revision.bIsShelve = Operation->IsShelve();

	FString OutFilename;
	InCommand.bCommandSuccessful = Revision.Get(OutFilename, ScopedConnection.GetConnection());

	if (InCommand.bCommandSuccessful)
	{
		Operation->SetOutPackageFilename(OutFilename);
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceGetFileWorker::UpdateStates() const
{
	return false;
}

FName FPerforceWhereWorker::GetName() const
{
	return "Where";
}

bool FPerforceWhereWorker::Execute(FPerforceSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceWhereWorker::Execute);

	FScopedPerforceConnection ScopedConnection(InCommand);

	if (!InCommand.IsCanceled() && ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		TSharedRef<FWhere, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FWhere>(InCommand.Operation);

		TArray<FString> Parameters;
		Parameters.Append(InCommand.Files);

		FP4RecordSet Records;
		InCommand.bCommandSuccessful = Connection.RunCommand(TEXT("where"), Parameters, Records, InCommand.ResultInfo, FOnIsCancelled::CreateRaw(&InCommand, &FPerforceSourceControlCommand::IsCanceled), InCommand.bConnectionDropped);

		if (InCommand.bCommandSuccessful)
		{
			TArray<FWhere::FileInfo> Files;
			Files.Reserve(Records.Num());

			for (FP4Record& P4Record : Records)
			{
				const FString* LocalPath = P4Record.Find("path");
				const FString* RemotePath = P4Record.Find("depotFile");

				if (ensure(LocalPath) && ensure(RemotePath))
				{
					FWhere::FileInfo FileInfo;
					FileInfo.LocalPath = FPaths::CreateStandardFilename(*LocalPath);
					FileInfo.RemotePath = *RemotePath;
					Files.Emplace(MoveTemp(FileInfo));
				}
			}

			Operation->SetFiles(MoveTemp(Files));
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPerforceWhereWorker::UpdateStates() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
