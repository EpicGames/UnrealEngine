// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPerforceSourceControlWorker.h"
#include "PerforceSourceControlState.h"
#include "PerforceSourceControlChangelistState.h"

class FPerforceSourceControlRevision;
class FPerforceSourceControlCommand;
typedef TMap<FString, TArray< TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> > > FPerforceFileHistoryMap;

class FPerforceConnectWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceConnectWorker(FPerforceSourceControlProvider& InSourceControlProvider) 
		: IPerforceSourceControlWorker(InSourceControlProvider) 
	{}
	virtual ~FPerforceConnectWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceCheckOutWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceCheckOutWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
		, InChangelist(FPerforceSourceControlChangelist::DefaultChangelist) // By default, add checked out files in the default changelist.
	{}
	virtual ~FPerforceCheckOutWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
	FPerforceSourceControlChangelist InChangelist;
};

class FPerforceCheckInWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceCheckInWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceCheckInWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;

	/** Changelist we asked to submit */
	FPerforceSourceControlChangelist InChangelist;

	/** Changelist we submitted */
	FPerforceSourceControlChangelist OutChangelist;

	/** List of file to kept in checked out after the submit.*/
	TMap<FString, EPerforceState::Type> FilesToKeepCheckedOut;
};

class FPerforceGetFileListWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetFileListWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetFileListWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
};

class FPerforceMarkForAddWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceMarkForAddWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
		, InChangelist(FPerforceSourceControlChangelist::DefaultChangelist) // By default, add new files in the default changelist.
	{}
	virtual ~FPerforceMarkForAddWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
	FPerforceSourceControlChangelist InChangelist;
};

class FPerforceDeleteWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceDeleteWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
		, Changelist(FPerforceSourceControlChangelist::DefaultChangelist) // By default, add deleted files in the default changelist
	{}
	virtual ~FPerforceDeleteWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
	FPerforceSourceControlChangelist Changelist;
};

class FPerforceRevertWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceRevertWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceRevertWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;

	/** Changelist to be udpated */
	FPerforceSourceControlChangelist ChangelistToUpdate;
};

class FPerforceSyncWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceSyncWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceSyncWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
};

class FPerforceUpdateStatusWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceUpdateStatusWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceUpdateStatusWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FPerforceSourceControlState> OutStates;

	/** Map of filename->state */
	TMap<FString, EPerforceState::Type> OutStateMap;

	/** Map of filenames to history */
	FPerforceFileHistoryMap OutHistory;

	/** Map of filenames to modified flag */
	TArray<FString> OutModifiedFiles;

	/** Override on status update return */
	bool bForceQuiet = false;
};

class FPerforceGetWorkspacesWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetWorkspacesWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetWorkspacesWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceGetProjectWorkspacesWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetProjectWorkspacesWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetProjectWorkspacesWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceGetPendingChangelistsWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetPendingChangelistsWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetPendingChangelistsWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FPerforceSourceControlChangelistState> OutChangelistsStates;
	TArray<TArray<FPerforceSourceControlState>> OutCLFilesStates;
	TArray<TMap<FString, EPerforceState::Type>> OutCLShelvedFilesStates;
	TArray<TMap<FString, FString>> OutCLShelvedFilesMap;

private:
	/** Controls whether or not we will remove changelists from the cache after a full update */
	bool bCleanupCache = false;
};

class FPerforceCopyWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceCopyWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceCopyWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;
};

class FPerforceResolveWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceResolveWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}

	virtual ~FPerforceResolveWorker() = default;
	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
private:
	TArray< FString > UpdatedFiles;
};

class FPerforceChangeStatusWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceChangeStatusWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceChangeStatusWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceNewChangelistWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceNewChangelistWorker(FPerforceSourceControlProvider& InSourceControlProvider);
	virtual ~FPerforceNewChangelistWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** New changelist information */
	FPerforceSourceControlChangelist NewChangelist;
	FPerforceSourceControlChangelistState NewChangelistState;

	/** Files that were moved */
	TArray<FString> MovedFiles;
};

class FPerforceDeleteChangelistWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceDeleteChangelistWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceDeleteChangelistWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	FPerforceSourceControlChangelist DeletedChangelist;
};

class FPerforceEditChangelistWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceEditChangelistWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceEditChangelistWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	FPerforceSourceControlChangelist EditedChangelist;
	FText EditedDescription;
};

class FPerforceRevertUnchangedWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceRevertUnchangedWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceRevertUnchangedWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
protected:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;

	/** Changelist to be updated */
	FPerforceSourceControlChangelist ChangelistToUpdate;
};

class FPerforceReopenWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceReopenWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceReopenWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

protected:
	/** Reopened files */
	TArray<FString> ReopenedFiles;
	
	/** Destination changelist */
	FPerforceSourceControlChangelist DestinationChangelist;
};

class FPerforceShelveWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceShelveWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceShelveWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

protected:
	/** Map of filenames to perforce state */
	TMap<FString, EPerforceState::Type> OutResults;

	/** Map depot filenames to local file */
	TMap<FString, FString> OutFileMap;

	/** Reopened files */
	TArray<FString> MovedFiles;

	/** Changelist description if needed */
	FString ChangelistDescription;

	/** Changelist(s) to be updated */
	FPerforceSourceControlChangelist InChangelistToUpdate;
	FPerforceSourceControlChangelist OutChangelistToUpdate;
};

class FPerforceDeleteShelveWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceDeleteShelveWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceDeleteShelveWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

protected:
	/** List of files to remove from shelved files in changelist state */
	TArray<FString> FilesToRemove;

	/** Changelist to be updated */
	FPerforceSourceControlChangelist ChangelistToUpdate;
};

class FPerforceUnshelveWorker : public IPerforceSourceControlWorker
{
public:
	FPerforceUnshelveWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceUnshelveWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

protected:
	/** Changelist to be updated */
	FPerforceSourceControlChangelist ChangelistToUpdate;

	/** List of files states after update */
	TArray<FPerforceSourceControlState> ChangelistFilesStates;
};

class FPerforceDownloadFileWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceDownloadFileWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceDownloadFileWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceRevisionInfoWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceRevisionInfoWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceRevisionInfoWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceCreateWorkspaceWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceCreateWorkspaceWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceCreateWorkspaceWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

private:
	void AddType(const class FCreateWorkspace& Operation, FStringBuilderBase& ClientDesc);
};

class FPerforceDeleteWorkspaceWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceDeleteWorkspaceWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceDeleteWorkspaceWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceGetChangelistDetailsWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetChangelistDetailsWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetChangelistDetailsWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceGetFileWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceGetFileWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceGetFileWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};

class FPerforceWhereWorker final : public IPerforceSourceControlWorker
{
public:
	FPerforceWhereWorker(FPerforceSourceControlProvider& InSourceControlProvider)
		: IPerforceSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPerforceWhereWorker() = default;

	// IPerforceSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPerforceSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
};