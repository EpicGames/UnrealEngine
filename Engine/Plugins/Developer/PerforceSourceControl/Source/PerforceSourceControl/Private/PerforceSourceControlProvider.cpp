// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlProvider.h"

#include "Algo/Transform.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "ISourceControlLabel.h"
#include "ISourceControlModule.h"
#include "Logging/MessageLog.h"
#include "Misc/CommandLine.h"
#include "Misc/MessageDialog.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/ScopeRWLock.h"
#include "PerforceConnection.h"
#include "PerforceConnectionInfo.h"
#include "PerforceSourceControlCommand.h"
#include "PerforceSourceControlLabel.h"
#include "ScopedSourceControlProgress.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "SPerforceSourceControlSettings.h"

static FName ProviderName("Perforce");

#define LOCTEXT_NAMESPACE "PerforceSourceControl"

namespace 
{
	/** Amount of seconds an idle persistent connection can remain open before the provider closes it.
	 * This lowers the number of open connections to a perforce server across a studio
	 * to conserve memory usage of the server.
	 */
	float IdleConnectionDisconnectSeconds = 60 * 60.0; // 1 hour

	static FAutoConsoleVariableRef CVarIdleConnectionDisconnectSeconds(
		TEXT("SourceControl.Perforce.IdleConnectionDisconnectSeconds"),
		IdleConnectionDisconnectSeconds,
		TEXT("The number of seconds a perforce connection will be kept open without activity before being automatically disconnected"),
		ECVF_Default);

	void ParseCmdLineSetting(FSourceControlInitSettings& Settings)
	{
		if (!Settings.ShouldReadFromCmdLine())
		{
			return;
		}

		auto ParseCmdLineSetting = [&Settings](const TCHAR* SettingKey) -> void
			{
				FString SettingValue;
				if (FParse::Value(FCommandLine::Get(), *WriteToString<64>(SettingKey, TEXT("=")), SettingValue))
				{
					Settings.AddSetting(SettingKey, SettingValue);
				}
			};

		ParseCmdLineSetting(TEXT("P4Port"));
		ParseCmdLineSetting(TEXT("P4User"));
		ParseCmdLineSetting(TEXT("P4Client"));
		ParseCmdLineSetting(TEXT("P4Host"));
		ParseCmdLineSetting(TEXT("P4Passwd"));
		ParseCmdLineSetting(TEXT("P4Changelist"));
	}
}

FPerforceSourceControlProvider::FPerforceSourceControlProvider()
	: PerforceSCCSettings(*this, FStringView())
	, InitialSettings(FSourceControlInitSettings::EBehavior::OverrideExisting, FSourceControlInitSettings::ECmdLineFlags::ReadAll)
	, OwnerName(TEXT("Default"))
	, bServerAvailable(false)
	, bIsWorkspaceValidForProject(true)
	, bLoginError(false)
	, PersistentConnection(nullptr)
{
	ParseCmdLineSetting(InitialSettings);

	AccessSettings().LoadSettings();
}

FPerforceSourceControlProvider::FPerforceSourceControlProvider(const FStringView& InOwnerName, const FSourceControlInitSettings& InInitialSettings)
	: PerforceSCCSettings(*this, InOwnerName)
	, InitialSettings(InInitialSettings)
	, OwnerName(InOwnerName)
	, bServerAvailable(false)
	, bIsWorkspaceValidForProject(true)
	, bLoginError(false)
	, PersistentConnection(nullptr)
{
	
	ParseCmdLineSetting(InitialSettings);

	AccessSettings().SetAllowSave(InInitialSettings.CanWriteToConfigFile());
	AccessSettings().SetAllowLoad(InInitialSettings.CanReadFromConfigFile());

	AccessSettings().LoadSettings();
}

void FPerforceSourceControlProvider::Init(bool bForceConnection)
{
	const EInitFlags Flags = bForceConnection ? EInitFlags::AttemptConnection : EInitFlags::None;

	ParseCommandLineSettings(Flags);
}

ISourceControlProvider::FInitResult FPerforceSourceControlProvider::Init(EInitFlags Flags)
{
	FInitResult Result = ParseCommandLineSettings(Flags);
	Result.bIsAvailable = IsAvailable();

	return Result;
}

void FPerforceSourceControlProvider::Close()
{
	if ( PersistentConnection )
	{
		PersistentConnection->Disconnect();
		PersistentConnection.Reset();
	}

	// clear the cache
	{
		FWriteScopeLock WriteToStateCacheLock(StateCacheLock);
		StateCache.Empty();
	}

	bServerAvailable = false;
}

TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> FPerforceSourceControlProvider::GetStateInternal(const FString& Filename)
{
	TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe>* State;
	{
		FReadScopeLock ReadFromStateCacheLock(StateCacheLock);
		State = StateCache.Find(Filename);
	}

	if(State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		FWriteScopeLock WriteToStateCacheLock(StateCacheLock);

		// cache an unknown state for this item
		TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> NewState = MakeShared<FPerforceSourceControlState>(Filename);
		StateCache.Add(Filename, NewState);
		return NewState;
	}
}

TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> FPerforceSourceControlProvider::GetStateInternal(const FPerforceSourceControlChangelist& InChangelist)
{
	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe>* State;
	{
		FReadScopeLock ReadFromChangelistsStateCacheLock(ChangelistsStateCacheLock);
		State = ChangelistsStateCache.Find(InChangelist);
	}

	if (State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		FWriteScopeLock WriteToChangelistsStateCacheLock(ChangelistsStateCacheLock);

		// cache an unknown state for this item
		TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> NewState = MakeShared<FPerforceSourceControlChangelistState>(InChangelist);
		ChangelistsStateCache.Add(InChangelist, NewState);
		return NewState;
	}
}

FText FPerforceSourceControlProvider::GetStatusText() const
{
	const FPerforceSourceControlSettings& Settings = AccessSettings();

	FFormatNamedArguments Args;
	Args.Add( TEXT("IsEnabled"), IsEnabled() ? LOCTEXT("Yes", "Yes") : LOCTEXT("No", "No") );
	Args.Add( TEXT("IsConnected"), (IsEnabled() && IsAvailable()) ? LOCTEXT("Yes", "Yes") : LOCTEXT("No", "No") );
	Args.Add( TEXT("PortNumber"), FText::FromString( Settings.GetPort() ) );
	Args.Add( TEXT("UserName"), FText::FromString( Settings.GetUserName() ) );
	Args.Add( TEXT("ClientSpecName"), FText::FromString( Settings.GetWorkspace() ) );

	FText FormattedError;
	TArray<FText> RecentErrors = GetLastErrors();
	if (RecentErrors.Num() > 0)
	{
		FFormatNamedArguments ErrorArgs;
		ErrorArgs.Add( TEXT("ErrorText"), RecentErrors[0] );

		FormattedError = FText::Format( LOCTEXT("PerforceErrorStatusText", "Error: {ErrorText}\n\n"), ErrorArgs );
	}

	Args.Add( TEXT("ErrorText"), FormattedError);

	return FText::Format( LOCTEXT("PerforceStatusText", "{ErrorText}Enabled: {IsEnabled}\nConnected: {IsConnected}\n\nPort: {PortNumber}\nUser name: {UserName}\nClient name: {ClientSpecName}"), Args );
}

TMap<ISourceControlProvider::EStatus, FString> FPerforceSourceControlProvider::GetStatus() const
{
	const FPerforceSourceControlSettings& Settings = AccessSettings();
	TMap<EStatus, FString> Result;
	Result.Add(EStatus::Enabled, IsEnabled() ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::Connected, (IsEnabled() && IsAvailable()) ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::Port, Settings.GetPort());
	Result.Add(EStatus::User, Settings.GetUserName());
	Result.Add(EStatus::Client, Settings.GetWorkspace());

	if (!Settings.GetWorkspace().IsEmpty())
	{
		FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, *const_cast<FPerforceSourceControlProvider*>(this));
		if (ScopedConnection.IsValid())
		{
			FPerforceConnection& Connection = ScopedConnection.GetConnection();
			Result.Add(EStatus::WorkspacePath, Connection.ClientRoot);
		}
	}
	if (!Settings.GetChangelistNumber().IsEmpty())
	{
		Result.Add(EStatus::Changeset, Settings.GetChangelistNumber());
	}

	return Result;
}

bool FPerforceSourceControlProvider::IsEnabled() const
{
	return true;
}

bool FPerforceSourceControlProvider::IsAvailable() const
{
	return bServerAvailable && !IsLoginError() && bIsWorkspaceValidForProject;
}

bool FPerforceSourceControlProvider::EstablishPersistentConnection(FSourceControlResultInfo* OutResultInfo)
{
	FPerforceConnectionInfo ConnectionInfo = AccessSettings().GetConnectionInfo();

	bool bIsValidConnection = false;
	if (!PersistentConnection)
	{
		PersistentConnection = MakeUnique<FPerforceConnection>(ConnectionInfo, *this, OutResultInfo);
	}

	bIsValidConnection = PersistentConnection->IsValidConnection();
	if (!bIsValidConnection)
	{
		PersistentConnection.Reset();
		PersistentConnection = MakeUnique<FPerforceConnection>(ConnectionInfo, *this, OutResultInfo);
		bIsValidConnection = PersistentConnection->IsValidConnection();
	}

	bServerAvailable = bIsValidConnection;
	return bIsValidConnection;
}

void FPerforceSourceControlProvider::ResetPersistentConnection()
{
	PersistentConnection.Reset();
}

ISourceControlProvider::FInitResult FPerforceSourceControlProvider::ParseCommandLineSettings(EInitFlags InitFlags)
{
	ISourceControlProvider::FInitResult Result;

	FPerforceSourceControlSettings& P4Settings = AccessSettings();

	// First we take a copy of the existing settings
	FString PortName = P4Settings.GetPort();
	FString UserName = P4Settings.GetUserName();
	FString ClientSpecName = P4Settings.GetWorkspace();
	FString HostOverrideName = P4Settings.GetHostOverride();
	FString Changelist = P4Settings.GetChangelistNumber();

	EConnectionOptions ConnectionOptions = EConnectionOptions::None;

	// Then we see if any of these settings are overridden by the initial settings
	// Note that as long as one setting is overridden, we will reset all non-overridden 
	// values to empty and later calculate them via FPerforceConnection::EnsureValidConnection
	if (InitialSettings.HasOverrides())
	{
		InitialSettings.OverrideSetting(TEXT("P4Port"),			PortName);
		InitialSettings.OverrideSetting(TEXT("P4User"),			UserName);
		InitialSettings.OverrideSetting(TEXT("P4Client"),		ClientSpecName);
		InitialSettings.OverrideSetting(TEXT("P4Host"),			HostOverrideName);
		InitialSettings.OverrideSetting(TEXT("P4Passwd"),		Ticket);
		InitialSettings.OverrideSetting(TEXT("P4Changelist"),	Changelist);

		// If P4Client is overridden then we can rely on that, even if it is blank (no workspace) and 
		// so don't need to automatically find a workspace when ensuring the connection.
		if (InitialSettings.IsOverridden(TEXT("P4Client")))
		{
			ConnectionOptions |= EConnectionOptions::WorkspaceOptional;
		}

		P4Settings.SetPort(PortName);
		P4Settings.SetUserName(UserName);
		P4Settings.SetWorkspace(ClientSpecName);
		P4Settings.SetHostOverride(HostOverrideName);
		P4Settings.SetChangelistNumber(Changelist);
	}

	if (EnumHasAnyFlags(InitFlags, EInitFlags::AttemptConnection))
	{
		bLoginError = false;

		FPerforceConnectionInfo ConnectionInfo = P4Settings.GetConnectionInfo();
		FPerforceConnectionInfo OutputSettings;

		if (EnumHasAnyFlags(InitFlags, EInitFlags::SuppressErrorLogging))
		{
			ConnectionOptions |= EConnectionOptions::SuppressErrorLogging;
		}

		if(FPerforceConnection::EnsureValidConnection(ConnectionInfo, *this, ConnectionOptions, OutputSettings, Result.Errors))
		{
			// The connection was a success so we should store the values used by the successful connection
			P4Settings.SetPort(OutputSettings.Port);
			P4Settings.SetUserName(OutputSettings.UserName);
			P4Settings.SetWorkspace(OutputSettings.Workspace);

			bServerAvailable = true;
		}

		// Fill in FInitResult::ConnectionSettings with the actual settings that were used

		if (!OutputSettings.Port.IsEmpty())
		{
			Result.ConnectionSettings.Add(ISourceControlProvider::EStatus::Port, OutputSettings.Port);
		}

		if (!OutputSettings.UserName.IsEmpty())
		{
			Result.ConnectionSettings.Add(ISourceControlProvider::EStatus::User, OutputSettings.UserName);
		}

		if (!OutputSettings.Workspace.IsEmpty())
		{
			Result.ConnectionSettings.Add(ISourceControlProvider::EStatus::Client, OutputSettings.Workspace);
		}
	}

	//Save off settings so this doesn't happen every time
	SaveConnectionSettings();

	return Result;
}

const FString& FPerforceSourceControlProvider::GetTicket() const
{
	return Ticket;
}

const FString& FPerforceSourceControlProvider::GetOwnerName() const
{
	return OwnerName;
}

const FName& FPerforceSourceControlProvider::GetName() const
{
	return ProviderName;
}

void FPerforceSourceControlProvider::SetLastErrors(const TArray<FText>& InErrors)
{
	static FString SessionExpiredMessage(TEXT("Your session has expired, please login again.\n"));

	bool bContainsLoginError = false;
	for (const FText& It : InErrors)
	{
		if (It.ToString() == SessionExpiredMessage)
		{
			bContainsLoginError = true;
			break;
		}
	}

	bLoginError = bContainsLoginError;

	FScopeLock Lock(&LastErrorsCriticalSection);
	LastErrors = InErrors;
}

bool FPerforceSourceControlProvider::IsLoginError() const
{
	return bLoginError;
}

void FPerforceSourceControlProvider::SetIsWorkspaceValidForProject(const bool bInIsWorkspaceValidForProject)
{
	bIsWorkspaceValidForProject = bInIsWorkspaceValidForProject;
}

TArray<FText> FPerforceSourceControlProvider::GetLastErrors() const
{
	FScopeLock Lock(&LastErrorsCriticalSection);
	TArray<FText> Result = LastErrors;
	return Result;
}

int32 FPerforceSourceControlProvider::GetNumLastErrors() const
{
	FScopeLock Lock(&LastErrorsCriticalSection);
	return LastErrors.Num();
}

ECommandResult::Type FPerforceSourceControlProvider::GetState( const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> >& OutState, EStateCacheUsage::Type InStateCacheUsage )
{
	if(!IsEnabled())
	{
		return ECommandResult::Failed;
	}

	TArray<FString> AbsoluteFiles = SourceControlHelpers::AbsoluteFilenames(InFiles);

	if(InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		Execute(ISourceControlOperation::Create<FUpdateStatus>(), AbsoluteFiles);
	}

	for( TArray<FString>::TConstIterator It(AbsoluteFiles); It; It++)
	{
		OutState.Add(GetStateInternal(*It));
	}

	return ECommandResult::Succeeded;
}

ECommandResult::Type FPerforceSourceControlProvider::GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	if (!IsEnabled())
	{
		return ECommandResult::Failed;
	}

	if (InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		TSharedRef<class FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetChangelistsToUpdate(InChangelists);

		ISourceControlProvider::Execute(UpdatePendingChangelistsOperation, EConcurrency::Synchronous);
	}

	for (FSourceControlChangelistRef Changelist : InChangelists)
	{
		FPerforceSourceControlChangelistRef PerforceChangelist = StaticCastSharedRef<FPerforceSourceControlChangelist>(Changelist);
		OutState.Add(GetStateInternal(PerforceChangelist.Get()));
	}

	return ECommandResult::Succeeded;
}

bool FPerforceSourceControlProvider::RemoveFileFromCache(const FString& Filename)
{
	FWriteScopeLock WriteToStateCacheLock(StateCacheLock);
	return StateCache.Remove(Filename) > 0;
}

bool FPerforceSourceControlProvider::RemoveChangelistFromCache(const FPerforceSourceControlChangelist& Changelist)
{
	FWriteScopeLock WriteToChangelistsStateCacheLock(ChangelistsStateCacheLock);
	return ChangelistsStateCache.Remove(Changelist) > 0;
}

TArray<FSourceControlStateRef> FPerforceSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const
{
	FReadScopeLock ReadFromStateCacheLock(StateCacheLock);

	TArray<FSourceControlStateRef> Result;
	for (const auto& CacheItem : StateCache)
	{
		FSourceControlStateRef State = CacheItem.Value;
		if (Predicate(State))
		{
			Result.Add(State);
		}
	}
	return Result;
}

TArray<FSourceControlChangelistStateRef> FPerforceSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlChangelistStateRef&)> Predicate) const
{
	FReadScopeLock ReadFromChangelistsStateCacheLock(ChangelistsStateCacheLock);

	TArray<FSourceControlChangelistStateRef> Result;
	for (const auto& CacheItem : ChangelistsStateCache)
	{
		FSourceControlChangelistStateRef State = CacheItem.Value;
		if (Predicate(State))
		{
			Result.Add(State);
		}
	}
	return Result;
}

FDelegateHandle FPerforceSourceControlProvider::RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged )
{
	return OnSourceControlStateChanged.Add( SourceControlStateChanged );
}

void FPerforceSourceControlProvider::UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle )
{
	OnSourceControlStateChanged.Remove( Handle );
}

ECommandResult::Type FPerforceSourceControlProvider::Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InBaseChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate )
{
	if(!IsEnabled())
	{
		// Note that IsEnabled() always returns true so unless it is changed, this code will never be executed
		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	TSharedPtr<FPerforceSourceControlChangelist, ESPMode::ThreadSafe> InChangelist = StaticCastSharedPtr<FPerforceSourceControlChangelist>(InBaseChangelist);

	TArray<FString> AbsoluteFiles = SourceControlHelpers::AbsoluteFilenames(InFiles);

	// Query to see if the we allow this operation
	TSharedPtr<IPerforceSourceControlWorker, ESPMode::ThreadSafe> Worker = CreateWorker(InOperation->GetName());
	if(!Worker.IsValid())
	{
		// this operation is unsupported by this source control provider
		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("OperationName"), FText::FromName(InOperation->GetName()) );
		Arguments.Add( TEXT("ProviderName"), FText::FromName(GetName()) );
		FText Message = FText::Format(LOCTEXT("UnsupportedOperation", "Operation '{OperationName}' not supported by revision control provider '{ProviderName}'"), Arguments);

		FMessageLog("SourceControl").Error(Message);
		InOperation->AddErrorMessge(Message);

		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	FPerforceSourceControlChangelist Changelist = InChangelist ? InChangelist.ToSharedRef().Get() : FPerforceSourceControlChangelist();

	// fire off operation
	if(InConcurrency == EConcurrency::Synchronous)
	{
		FPerforceSourceControlCommand* Command = new FPerforceSourceControlCommand(InOperation, Worker.ToSharedRef());
		Command->bAutoDelete = false;
		Command->Files = AbsoluteFiles;
		Command->StatusBranchNames = StatusBranchNames;
		Command->ContentRoots = ContentRoots;
		Command->OperationCompleteDelegate = InOperationCompleteDelegate;
		Command->Changelist = Changelist;
		return ExecuteSynchronousCommand(*Command, InOperation->GetInProgressString(), true);
	}
	else
	{
		FPerforceSourceControlCommand* Command = new FPerforceSourceControlCommand(InOperation, Worker.ToSharedRef());
		Command->bAutoDelete = true;
		Command->Files = AbsoluteFiles;
		Command->StatusBranchNames = StatusBranchNames;
		Command->ContentRoots = ContentRoots;
		Command->OperationCompleteDelegate = InOperationCompleteDelegate;
		Command->Changelist = Changelist;
		return IssueCommand(*Command, false);
	}
}

bool FPerforceSourceControlProvider::CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const
{
	FPerforceSourceControlProvider* NonConstThis = const_cast<FPerforceSourceControlProvider*>(this);
	return IPerforceSourceControlWorker::CreateWorker(InOperation->GetName(), *NonConstThis).IsValid();
}

bool FPerforceSourceControlProvider::CanCancelOperation( const FSourceControlOperationRef& InOperation ) const
{
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FPerforceSourceControlCommand& Command = *CommandQueue[CommandIndex];
		if (Command.Operation == InOperation)
		{
			check(Command.bAutoDelete);
			return true;
		}
	}

	// operation was not in progress!
	return false;
}

void FPerforceSourceControlProvider::CancelOperation( const FSourceControlOperationRef& InOperation )
{
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FPerforceSourceControlCommand& Command = *CommandQueue[CommandIndex];
		if (Command.Operation == InOperation)
		{
			check(Command.bAutoDelete);
			Command.Cancel();
			return;
		}
	}
}

bool FPerforceSourceControlProvider::UsesLocalReadOnlyState() const
{
	return true;
}

bool FPerforceSourceControlProvider::UsesChangelists() const
{
	return true;
}

bool FPerforceSourceControlProvider::UsesUncontrolledChangelists() const
{
	return true;
}

bool FPerforceSourceControlProvider::UsesCheckout() const
{
	return true;
}

bool FPerforceSourceControlProvider::UsesFileRevisions() const
{
	return true;
}

bool FPerforceSourceControlProvider::UsesSnapshots() const
{
	return false;
}

bool FPerforceSourceControlProvider::UsesSoftRevertOnDelete() const
{
	return false;
}

bool FPerforceSourceControlProvider::AllowsDiffAgainstDepot() const
{
	return true;
}

TOptional<bool> FPerforceSourceControlProvider::HasChangesToSync() const
{
	return TOptional<bool>();
}

TOptional<bool> FPerforceSourceControlProvider::HasChangesToCheckIn() const
{
	return TOptional<bool>();
}

void FPerforceSourceControlProvider::OutputCommandMessages(const FPerforceSourceControlCommand& InCommand) const
{
	if (IsInGameThread()) // On the game thread we can use FMessageLog
	{
		FMessageLog SourceControlLog("SourceControl");

		if (InCommand.Operation->ShouldLogErrors())
		{
			for (int32 ErrorIndex = 0; ErrorIndex < InCommand.ResultInfo.ErrorMessages.Num(); ++ErrorIndex)
			{
				SourceControlLog.Error(FText::Format(LOCTEXT("OutputCommandMessagesFormatError", "CommandMessage Command: {0}, Error: {1}"), FText::FromName(InCommand.Operation->GetName()), InCommand.ResultInfo.ErrorMessages[ErrorIndex]));
			}
		}

		if (InCommand.Operation->ShouldLogInfos())
		{
			for (int32 InfoIndex = 0; InfoIndex < InCommand.ResultInfo.InfoMessages.Num(); ++InfoIndex)
			{
				SourceControlLog.Info(FText::Format(
					LOCTEXT("OutputCommandMessagesFormatInfo", "CommandMessage Command: {0}, Info: {1}"),
					FText::FromName(InCommand.Operation->GetName()),
					InCommand.ResultInfo.InfoMessages[InfoIndex]
				));
			}
		}
	}
	else // On background threads we must log directly as FMessageLog internals cannot be assumed to be thread safe
	{
		if (InCommand.Operation->ShouldLogErrors())
		{
			for (const FText& Error : InCommand.ResultInfo.ErrorMessages)
			{
				UE_LOGF(LogSourceControl, Error, "Command: %ls, Error: %ls", *InCommand.Operation->GetName().ToString(), *Error.ToString());
			}
		}

		for (const FText& Info : InCommand.ResultInfo.InfoMessages)
		{
			UE_LOGF(LogSourceControl, Log, "Command: %ls, Info: %ls", *InCommand.Operation->GetName().ToString(), *Info.ToString());
		}
	}
}

void FPerforceSourceControlProvider::Tick()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceSourceControlProvider::Tick);
	
	bool bStatesUpdated = false;
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FPerforceSourceControlCommand& Command = *CommandQueue[CommandIndex];
		if (Command.bExecuteProcessed)
		{
			// Remove command from the queue
			CommandQueue.RemoveAt(CommandIndex);

			// update connection state
			bServerAvailable = Command.bConnectionWasSuccessful && (!Command.bConnectionDropped || Command.bCancelled);

			// let command update the states of any files
			bStatesUpdated |= Command.Worker->UpdateStates();

			// dump any messages to output log
			OutputCommandMessages(Command);

			// If the command was cancelled while trying to connect, the operation complete delegate will already
			// have been called. Otherwise, now we have to call it.
			if (!Command.bCancelledWhileTryingToConnect)
			{
				Command.ReturnResults();
			}

			//commands that are left in the array during a tick need to be deleted
			if(Command.bAutoDelete)
			{
				// Only delete commands that are not running 'synchronously'
				delete &Command;
			}

			// only do one command per tick loop, as we dont want concurrent modification
			// of the command queue (which can happen in the completion delegate)
			break;
		}
		// If a cancel is detected before the server has connected, abort immediately.
		else if (Command.bCancelled && !Command.bConnectionWasSuccessful)
		{
			// Mark command as having been cancelled while trying to connect
			Command.CancelWhileTryingToConnect();

			// If this was a synchronous command, set it free so that it will be deleted automatically
			// when its (still running) thread finally finishes
			Command.bAutoDelete = true;

			Command.ReturnResults();
			break;
		}
	}
	
	if(bStatesUpdated)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceSourceControlProvider::Tick::BroadcastStateUpdate);
		OnSourceControlStateChanged.Broadcast();
	}

	// Close idle connections when they haven't seen any activity for IdleConnectionDisconnectSeconds
	// Disabled for commandlets, which are often long running processes
	if (PersistentConnection && !IsRunningCommandlet())
	{
		const double Now = FPlatformTime::Seconds();

		const double ElapsedSinceLastComm = Now - PersistentConnection->GetLatestCommuncationTime();
		
		if (ElapsedSinceLastComm > IdleConnectionDisconnectSeconds)
		{
			UE_LOGF(LogSourceControl, Display,
				"Persistent perforce connection has not been used in %0.f seconds. Dropping connection",
				ElapsedSinceLastComm);

			Close();
		}
	}
}

static void ParseGetLabelsResults(FPerforceSourceControlProvider& InSourceControlProvider, const FP4RecordSet& InRecords, TArray< TSharedRef<ISourceControlLabel> >& OutLabels)
{
	// Iterate over each record found as a result of the command, parsing it for relevant information
	for (const FP4Record& ClientRecord : InRecords)
	{
		const FString& LabelName = ClientRecord(TEXT("label"));
		if(LabelName.Len() > 0)
		{
			OutLabels.Add(MakeShared<FPerforceSourceControlLabel>(InSourceControlProvider, LabelName));
		}
	}
}

TArray< TSharedRef<ISourceControlLabel> > FPerforceSourceControlProvider::GetLabels(const FString& InMatchingSpec) const
{
	TArray< TSharedRef<ISourceControlLabel> > Labels;

	// const_cast to avoid changing the ISourceControlProvider API as it is hard to deprecate without causing derived types to give compiler errors.
	FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, *const_cast<FPerforceSourceControlProvider*>(this));
	if(ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FP4RecordSet Records;
		TArray<FString> Parameters;
		FSourceControlResultInfo ResultInfo;
		Parameters.Add(TEXT("-E"));
		Parameters.Add(InMatchingSpec);
		bool bConnectionDropped = false;
		if(Connection.RunCommand(TEXT("labels"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped))
		{
			// const_cast to avoid changing the ISourceControlProvider API as it is hard to deprecate without causing derived types to give compiler errors.
			ParseGetLabelsResults(*const_cast<FPerforceSourceControlProvider*>(this), Records, Labels);
		}
		else
		{
			for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
			{
				FMessageLog("SourceControl").Warning(FText::Format(LOCTEXT("GetLabelsErrorFormat", "GetLabels Warning: {0}"), ErrorMsg));
			}
		}
	}

	return Labels;
}

TArray<FSourceControlChangelistRef> FPerforceSourceControlProvider::GetChangelists(EStateCacheUsage::Type InStateCacheUsage)
{
	if (!IsEnabled())
	{
		return TArray<FSourceControlChangelistRef>();
	}

	if (InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		TSharedRef<class FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetUpdateAllChangelists(true);

		ISourceControlProvider::Execute(UpdatePendingChangelistsOperation, EConcurrency::Synchronous);
	}

	TArray<FSourceControlChangelistRef> Changelists;
	{
		FReadScopeLock ReadFromChangelistsStateCacheLock(ChangelistsStateCacheLock);
		Algo::Transform(ChangelistsStateCache, Changelists, [](const TPair<FPerforceSourceControlChangelist, TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe>>& Pair)
		{
			return MakeShared<FPerforceSourceControlChangelist, ESPMode::ThreadSafe>(Pair.Key);
		});
	}

	// NOTE: Sort in ascending number. If this behavior needs to be configurable, we could have 3-state enum param: 'default, ascending, descending'. For P4, the 'default'
	//       should be ascending. The changelists are source control agnostics, so sorting a changelist files is probably not a notion that should be leaked in the generic interface.
	Changelists.Sort([](const FSourceControlChangelistRef& Lhs, const FSourceControlChangelistRef& Rhs)
	{
		return static_cast<const FPerforceSourceControlChangelist&>(Lhs.Get()).ToInt() < static_cast<const FPerforceSourceControlChangelist&>(Rhs.Get()).ToInt();
	});

	return Changelists;
}

bool FPerforceSourceControlProvider::TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const TArray<FString>& InFiles)
{
	if (!IsEnabled())
	{
		return false;
	}

	TSharedPtr<IPerforceSourceControlWorker, ESPMode::ThreadSafe> Worker = CreateWorker(InOperation->GetName());
	if (!Worker.IsValid())
	{
		// This operation is unsupported by this source control provider
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("OperationName"), FText::FromName(InOperation->GetName()));
		Arguments.Add(TEXT("ProviderName"), FText::FromName(GetName()));
		FText Message = FText::Format(LOCTEXT("UnsupportedOperation", "Operation '{OperationName}' not supported by revision control provider '{ProviderName}'"), Arguments);

		InOperation->AddErrorMessge(Message);

		return false;
	}

	// Note that this method can safely be called from any thread works because we know that
	// a) We are not executing any delegates.
	// b) That the FDownloadFile operation does not change the stage of any files in 
	// source control and so will not affect any cached states.
	// c) We are not invoking any globals that might touch the slate UI such as FMessageLog

	FPerforceSourceControlCommand Command(InOperation, Worker.ToSharedRef());
	Command.bAutoDelete = false;
	Command.Files = SourceControlHelpers::AbsoluteFilenames(InFiles);
	Command.StatusBranchNames = StatusBranchNames;
	Command.ContentRoots = ContentRoots;

	// DoWork will use a shared connection, we need to call DoThreadedWork to make sure that 
	// we use our own connection for this background thread.
	Command.DoThreadedWork();

	// Sanity check to make sure we are not running a command that modifies the cached states from a background thread
	check(Command.Worker->UpdateStates() == false);

	if (!Command.bConnectionWasSuccessful && !Command.bCancelled)
	{
		Command.ResultInfo.OnConnectionFailed();
		
	}
	else if (Command.bConnectionDropped)
	{
		Command.ResultInfo.OnConnectionDroped();
	}

	OutputCommandMessages(Command);

	if (!Command.bCancelledWhileTryingToConnect)
	{
		Command.ReturnResults();
	}
	
	return Command.bCommandSuccessful;
}

ECommandResult::Type FPerforceSourceControlProvider::SwitchWorkspace(FStringView NewWorkspaceName, FSourceControlResultInfo& OutResultInfo, FString* OutOldWorkspaceName)
{
	if (!CommandQueue.IsEmpty())
	{
		UE_LOGF(LogSourceControl, Log, "Waiting on pending commands before switching workspace");

		// Run the busy loop while we wait for any current commands to be cleared.
		while (!CommandQueue.IsEmpty())
		{
			// Tick the command queue and update progress.
			Tick();

			// Sleep for a bit so we don't busy-wait so much.
			FPlatformProcess::Sleep(0.01f);
		}
	}

	
	Close();

	// Do not call Init directly as we do not want to save the new workspace name to
	// the source control settings!

	FPerforceSourceControlSettings& P4Settings = AccessSettings(); 
	
	const FString OldWorkspaceName = P4Settings.GetWorkspace();
	FString WorkspaceName = FString(NewWorkspaceName);

	if (!NewWorkspaceName.IsEmpty())
	{
		FPerforceConnectionInfo NewWorkspaceSettings = P4Settings.GetConnectionInfo();
		NewWorkspaceSettings.Workspace = WorkspaceName;

		ISourceControlProvider::FInitResult Results;

		FPerforceConnectionInfo OutputSettings;
		if (FPerforceConnection::EnsureValidConnection(NewWorkspaceSettings, *this, EConnectionOptions::WorkspaceOptional, OutputSettings, Results.Errors))
		{
			P4Settings.SetPort(OutputSettings.Port);
			P4Settings.SetUserName(OutputSettings.UserName);
			P4Settings.SetWorkspace(OutputSettings.Workspace);

			bServerAvailable = true;

			if (OutOldWorkspaceName != nullptr)
			{
				*OutOldWorkspaceName = OldWorkspaceName;
			}	
		}
		else
		{
			FText Message = FText::Format(LOCTEXT("Perforce_ConnectionFailed", "Failed to re-establish the connection after switching to workspace {0}"),
				FText::FromString(FString(NewWorkspaceName)));
			OutResultInfo.ErrorMessages.Add(Message);

			// The connection didn't work so we should try to restore the old workspace name
			P4Settings.SetWorkspace(OldWorkspaceName);

			return ECommandResult::Failed;
		}
	}
	else
	{
		// If we are just removing the workspace name then we don't need to ensure a valid connection, we can just go ahead and remove the name 
		// and continue with the existing settings.
		P4Settings.SetWorkspace(WorkspaceName);
	}

	UE_LOGF(LogSourceControl, Log, "Switched workspaces from '%ls' to '%ls'", *OldWorkspaceName, *WorkspaceName);

	return ECommandResult::Succeeded;
}

#if SOURCE_CONTROL_WITH_SLATE
TSharedRef<class SWidget> FPerforceSourceControlProvider::MakeSettingsWidget() const
{
	// const_cast to avoid changing the ISourceControlProvider API as it is hard to deprecate without
	// causing derived types to give compiler errors.
	return SNew(SPerforceSourceControlSettings, const_cast<FPerforceSourceControlProvider*>(this));
}
#endif

TUniquePtr<ISourceControlProvider> FPerforceSourceControlProvider::Create(const FStringView& InOwnerName, const FSourceControlInitSettings& InInitialSettings) const
{ 
	return MakeUnique<FPerforceSourceControlProvider>(InOwnerName, InInitialSettings);
}

const FPerforceSourceControlSettings& FPerforceSourceControlProvider::AccessSettings() const
{
	return PerforceSCCSettings;
}

FPerforceSourceControlSettings& FPerforceSourceControlProvider::AccessSettings()
{
	return PerforceSCCSettings;
}

void FPerforceSourceControlProvider::SaveConnectionSettings()
{
	PerforceSCCSettings.SaveSettings();
}

TSharedPtr<IPerforceSourceControlWorker, ESPMode::ThreadSafe> FPerforceSourceControlProvider::CreateWorker(const FName& InOperationName)
{
	return IPerforceSourceControlWorker::CreateWorker(InOperationName, *this);
}

ECommandResult::Type FPerforceSourceControlProvider::ExecuteSynchronousCommand(FPerforceSourceControlCommand& InCommand, const FText& Task, bool bSuppressResponseMsg)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPerforceSourceControlProvider::ExecuteSynchronousCommand);

	ECommandResult::Type Result = ECommandResult::Failed;

	struct Local
	{
		static void CancelCommand(FPerforceSourceControlCommand* InControlCommand)
		{
			InControlCommand->Cancel();
		}
	};

	FText TaskText = Task;
	// Display the progress dialog
	if (bSuppressResponseMsg)
	{
		TaskText = FText::GetEmpty();
	}
	FScopedSourceControlProgress Progress(TaskText, FSimpleDelegate::CreateStatic(&Local::CancelCommand, &InCommand));

	// Perform the command asynchronously
	IssueCommand( InCommand, false );

	// Wait until the command has been processed
	while (!InCommand.bCancelledWhileTryingToConnect && CommandQueue.Contains(&InCommand))
	{
		// Tick the command queue and update progress.
		Tick();

		Progress.Tick();

		// Sleep for a bit so we don't busy-wait so much.
		FPlatformProcess::Sleep(0.01f);
	}

	if (InCommand.bCancelled)
	{
		Result = ECommandResult::Cancelled;
	}
	else if (InCommand.bCommandSuccessful)
	{
		Result = ECommandResult::Succeeded;
	}

	// If the command failed, inform the user that they need to try again
	if (!InCommand.bCancelled && Result != ECommandResult::Succeeded && !bSuppressResponseMsg)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Perforce_ServerUnresponsive", "Perforce server is unresponsive. Please check your connection and try again."));
	}

	// Delete the command now if not marked as auto-delete
	if (!InCommand.bAutoDelete)
	{
		delete &InCommand;
	}

	return Result;
}

ECommandResult::Type FPerforceSourceControlProvider::IssueCommand(FPerforceSourceControlCommand& InCommand, const bool bSynchronous)
{
	if ( !bSynchronous && GThreadPool != NULL )
	{
		// Queue this to our worker thread(s) for resolving.
		// When asynchronous, any callback gets called from Tick().
		GThreadPool->AddQueuedWork(&InCommand);
		CommandQueue.Add(&InCommand);
		return ECommandResult::Succeeded;
	}
	else
	{
		InCommand.bCommandSuccessful = InCommand.DoWork();

		InCommand.Worker->UpdateStates();

		OutputCommandMessages(InCommand);

		return InCommand.ReturnResults();
	}
}

bool FPerforceSourceControlProvider::QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest)
{

	if (ConfigSrc.Len() == 0 || ConfigDest.Len() == 0)
	{
		return false;
	}

	// Request branch configuration from depot
	FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, *this);
	if (ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FP4RecordSet Records;
		TArray<FString> Parameters;
		FSourceControlResultInfo ResultInfo;
		Parameters.Add(TEXT("-o"));
		Parameters.Add(*ConfigDest);
		Parameters.Add(*ConfigSrc);

		FText GeneralErrorMessage = LOCTEXT("StatusBranchConfigGeneralFailure", "Unable to retrieve status branch configuration from depot");

		bool bConnectionDropped = false;
		if (Connection.RunCommand(TEXT("print"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped))
		{
			if (Records.Num() < 1 || Records[0][TEXT("depotFile")] != ConfigSrc)
			{
				FMessageLog("SourceControl").Error(GeneralErrorMessage);
				return false;
			}
		}
		else
		{
			FMessageLog("SourceControl").Error(GeneralErrorMessage);

			for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
			{
				FMessageLog("SourceControl").Error(ErrorMsg);
			}

			return false;
		}
	}
	else
	{
		FMessageLog("SourceControl").Error(LOCTEXT("StatusBranchConfigNoConnection", "Unable to retrieve status branch configuration from depot, no connection"));
		return false;
	}

	return true;
}

void FPerforceSourceControlProvider::RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRootIn)
{
	return RegisterStateBranches(BranchNames, TArray<FString> { ContentRootIn });
}

void FPerforceSourceControlProvider::RegisterStateBranches(const TArray<FString>& BranchNames, const TArray<FString>& ContentRootsIn)
{
	StatusBranchNames = BranchNames;
	ContentRoots = ContentRootsIn;
}

int32 FPerforceSourceControlProvider::GetStateBranchIndex(const FString& BranchName) const
{
	return StatusBranchNames.IndexOfByKey(BranchName);
}

bool FPerforceSourceControlProvider::GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const
{
	if (StatusBranchNames.IsValidIndex(BranchIndex))
	{
		OutBranchName = StatusBranchNames[BranchIndex];
		return !OutBranchName.IsEmpty();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
