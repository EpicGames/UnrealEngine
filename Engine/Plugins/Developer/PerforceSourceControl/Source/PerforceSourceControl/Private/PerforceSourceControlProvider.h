// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPerforceSourceControlWorker.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "PerforceSourceControlChangelistState.h"
#include "PerforceSourceControlSettings.h"
#include "PerforceSourceControlState.h"
#include "SourceControlInitSettings.h"

#include "DataStorage/Handles.h"
#include "HAL/CriticalSection.h"

class FPerforceConnection;
class FPerforceSourceControlCommand;

class FPerforceSourceControlProvider : public ISourceControlProvider
{
public:
	
	FPerforceSourceControlProvider();
	FPerforceSourceControlProvider(const FStringView& OwnerName, const FSourceControlInitSettings& InitialSettings);
	virtual ~FPerforceSourceControlProvider() = default;

	/* ISourceControlProvider implementation */
	virtual void Init(bool bForceConnection = true) override;
	virtual FInitResult Init(EInitFlags Flags) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual TMap<EStatus, FString> GetStatus() const override;
	virtual bool IsEnabled() const override;
	virtual bool IsAvailable() const override;
	virtual const FName& GetName(void) const override;
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) override;
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRootIn) override;
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const TArray<FString>& ContentRootsIn) override;
	virtual int32 GetStateBranchIndex(const FString& BranchName) const override;
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const override;
	virtual ECommandResult::Type GetState( const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage ) override;
	virtual ECommandResult::Type GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override;
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const override;
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged ) override;
	virtual void UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle ) override;
	virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() ) override;
	virtual bool CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual bool CanCancelOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual void CancelOperation( const FSourceControlOperationRef& InOperation ) override;
	virtual bool UsesLocalReadOnlyState() const override;
	virtual bool UsesChangelists() const override;
	virtual bool UsesUncontrolledChangelists() const override;
	virtual bool UsesCheckout() const override;
	virtual bool UsesFileRevisions() const override;
	virtual bool UsesSnapshots() const override;
	virtual bool UsesSoftRevertOnDelete() const override;
	virtual bool AllowsDiffAgainstDepot() const override;
	virtual TOptional<bool> HasChangesToSync() const override;
	virtual TOptional<bool> HasChangesToCheckIn() const override;
	virtual void Tick() override;
	virtual TArray< TSharedRef<class ISourceControlLabel> > GetLabels( const FString& InMatchingSpec ) const override;
	virtual TArray<FSourceControlChangelistRef> GetChangelists( EStateCacheUsage::Type InStateCacheUsage ) override;

	virtual bool TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const TArray<FString>& InFiles) override;
	virtual ECommandResult::Type SwitchWorkspace(FStringView NewWorkspaceName, FSourceControlResultInfo& OutResultInfo, FString* OutOldWorkspaceName) override;

#if SOURCE_CONTROL_WITH_SLATE
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const override;
#endif

	using ISourceControlProvider::Execute;

	/** Get the P4 ticket we will use for connections */
	const FString& GetTicket() const;

	/** Returns the name of the system that owns the provider */
	const FString& GetOwnerName() const;

	/** Set list of error messages that occurred after last perforce command */
	void SetLastErrors(const TArray<FText>& InErrors);

	/** Did most recent command generate a login error */
	bool IsLoginError() const;

	/** Sets whether or not the current workspace maps to the current project directory */
	void SetIsWorkspaceValidForProject(const bool bInIsWorkspaceValidForProject);

	/** Get list of error messages that occurred after last perforce command */
	TArray<FText> GetLastErrors() const;

	/** Get number of error messages seen after running last perforce command */
	int32 GetNumLastErrors() const;

	/** Helper function used to update state cache */
	TSharedRef<FPerforceSourceControlState, ESPMode::ThreadSafe> GetStateInternal(const FString& InFilename);

	/** Helper function used to update changelists state cache */
	TSharedRef<FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> GetStateInternal(const FPerforceSourceControlChangelist& InChangelist);

	/**
	 * Connects to the source control server if the persistent connection is not already established.
	 *
	 * @param OutResultInfo Optional structure to store errors encountered while creating the connection.
	 *                      When set to nullptr, errors will be immediately logged.
	 *
	 * @return true if the connection is established or became established and false if the connection failed.
	 */
	bool EstablishPersistentConnection(FSourceControlResultInfo* OutResultInfo);

	void ResetPersistentConnection();

	/** Get the persistent connection, if any */
	class FPerforceConnection* GetPersistentConnection()
	{
		return PersistentConnection.Get();
	}

	/** Remove a named file from the state cache */
	bool RemoveFileFromCache(const FString& Filename);

	/** Remove a changelist from the state cache */
	bool RemoveChangelistFromCache(const FPerforceSourceControlChangelist& Changelist);

	/** Returns a list of changelists from the cache based on a given predicate */
	TArray<FSourceControlChangelistStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlChangelistStateRef&)> Predicate) const;

	/** Returns the settings for the current source control provider */
	const FPerforceSourceControlSettings& AccessSettings() const;

	/** Returns the settings for the current source control provider */
	FPerforceSourceControlSettings& AccessSettings();

private:
	/** Instantiates and returns a new FPerforceSourceControlProvider */
	virtual TUniquePtr<ISourceControlProvider> Create(const FStringView& OwnerName, const FSourceControlInitSettings& InitialSettings) const override;

	void SaveConnectionSettings();

	/** Helper function used to create a worker for a particular operation */
	TSharedPtr<class IPerforceSourceControlWorker, ESPMode::ThreadSafe> CreateWorker(const FName& InOperationName);

	/** 
	 * Logs any messages that a command needs to output.
	 */
	void OutputCommandMessages(const FPerforceSourceControlCommand& InCommand) const;

	/**
	 * Loads user/SCC information from the INI file and can attempt to make a connection to the server if requested by
	 * the Flags parameter.
	 */
	ISourceControlProvider::FInitResult ParseCommandLineSettings(EInitFlags Flags);

	/**
	 * Helper function for running command 'synchronously'.
	 * This really doesn't execute synchronously; rather it adds the command to the queue & does not return until
	 * the command is completed.
	 */
	ECommandResult::Type ExecuteSynchronousCommand(FPerforceSourceControlCommand& InCommand, const FText& Task, bool bSuppressResponseMsg);

	/**
	 * Run a command synchronously or asynchronously.
	 */
	ECommandResult::Type IssueCommand(FPerforceSourceControlCommand& InCommand, const bool bSynchronous);

private:

	/** The settings for Perforce source control */
	FPerforceSourceControlSettings PerforceSCCSettings;

	/** The initial settings for the provider. These are used every time ::Init is called */
	FSourceControlInitSettings InitialSettings;

	/** Name of the system that owns the provider */
	FString OwnerName;

	/** The ticket we use for login. */
	FString Ticket;

	/** the root of the workspace we are currently using */
	FString WorkspaceRoot;

	/** Indicates if source control integration is available or not. */
	bool bServerAvailable;

	/** Whether or not the current workspace maps to the current project directory; this is true unless explicitly cleared */
	TAtomic<bool> bIsWorkspaceValidForProject;

	/** Saw login error when running last command. */
	TAtomic<bool> bLoginError;

	/** Critical section for thread safety of error messages that occurred after last perforce command */
	mutable FCriticalSection LastErrorsCriticalSection;

	/** List of error messages that occurred after last perforce command */
	TArray<FText> LastErrors;

	/** A pointer to the persistent P4 connection for synchronous operations */
	TUniquePtr<FPerforceConnection> PersistentConnection;

	/** State cache */
	TMap<FString, TSharedRef<class FPerforceSourceControlState, ESPMode::ThreadSafe> > StateCache;
	mutable FRWLock StateCacheLock;

	TMap<FPerforceSourceControlChangelist, TSharedRef<class FPerforceSourceControlChangelistState, ESPMode::ThreadSafe> > ChangelistsStateCache;
	mutable FRWLock ChangelistsStateCacheLock;

	/** Queue for commands given by the main thread */
	TArray <FPerforceSourceControlCommand* > CommandQueue;

	/** For notifying when the source control states in the cache have changed */
	FSourceControlStateChanged OnSourceControlStateChanged;

	/** Array of branch names for status queries */
	TArray<FString> StatusBranchNames;

	/** Content roots for branch status query mapping */
	TArray<FString> ContentRoots;

};
