// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlRevision.h"

#include "Logging/MessageLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "PerforceConnection.h"
#include "PerforceMessageLog.h"
#include "PerforceSourceControlProvider.h"

#define LOCTEXT_NAMESPACE "PerforceSourceControl"

FPerforceSourceControlRevision:: FPerforceSourceControlRevision(FPerforceSourceControlProvider& InSCCProvider)
	: RevisionNumber(0)
	, Date(0)
	, ChangelistNumber(0)
	, FileSize(0)
	, bIsShelve(false)
	, SCCProvider(InSCCProvider)
{
}

bool FPerforceSourceControlRevision::Get(FString& InOutFilename, EConcurrency::Type InConcurrency) const
{
	FScopedPerforceConnection ScopedConnection(InConcurrency, GetSCCProvider());
	if (ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();

		return Get(InOutFilename, Connection);
	}
	else
	{
		return false;
	}
}

bool FPerforceSourceControlRevision::Get(FString& InOutFilename, FPerforceConnection& Connection) const
{
	if (!Connection.IsValidConnection())
	{
		return false;
	}

	FP4RecordSet Records;
	bool bConnectionDropped;
	FSourceControlResultInfo ResultInfo;
	TArray<FString> Parameters;

	// Suppress the one-line file header normally added by Perforce.
	Parameters.Add(TEXT("-q"));

	// Make temp filename to 'print' to
	FString RevString;
	FString RevParam;

	if (bIsShelve)
	{
		RevString = FString::Printf(TEXT("%d"), ChangelistNumber);
		RevParam = TEXT("@=") + RevString;
	}
	else
	{
		RevString = (RevisionNumber < 0) ? TEXT("head") : FString::Printf(TEXT("%d"), RevisionNumber);
		RevParam = TEXT("#") + RevString;
	}

	FString AbsoluteFileName;
	if (InOutFilename.Len() > 0)
	{
		AbsoluteFileName = InOutFilename;
	}
	else
	{
		const FString File = FString::Printf(TEXT("%s-Rev-%s-"), *FPaths::GetBaseFilename(FileName), *RevString);
		const FString Extension = TEXT(".") + FPaths::GetExtension(FileName);
		const FString TempFileName = FPaths::CreateTempFilename(*FPaths::DiffDir(), *File, *Extension);
		AbsoluteFileName = FPaths::ConvertRelativePathToFull(TempFileName);
	}

	// output to file
	Parameters.Add(FString("-o") + AbsoluteFileName);
	Parameters.Add(FileName + RevParam);

	const bool bCmdResult = Connection.RunCommand(TEXT("print"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped);
	if (bCmdResult)
	{
		InOutFilename = AbsoluteFileName;
		return true;
	}
	else
	{
		for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
		{
			FTSMessageLog("SourceControl").Error(FText::Format(LOCTEXT("PerforceSourceControlRevisionGetErrorFormat", "FPerforceSourceControlRevision::Get print Error: {0}"), ErrorMsg));
		}

		return false;
	}
}

/** 
 * Helper function for ParseAnnotationResults, so we can get the user that last changed a file 
 * @param	ChangeNumber	The changelist number we want information about (of a particular line).
 * @param	InConnection	The Perforce connection to use
 * @returns the user that changed a particular line.
 */
static FString GetUserFromChangelist(int32 ChangeNumber, FPerforceConnection& InConnection)
{
	FP4RecordSet Records;
	bool bConnectionDropped;
	FSourceControlResultInfo ResultInfo;
	TArray<FString> Parameters;

	// Only describe the basic changelist information, suppress output of the file diffs
	Parameters.Add(TEXT("-s"));
	Parameters.Add(FString::Printf(TEXT("%d"), ChangeNumber));

	if(InConnection.RunCommand(TEXT("describe"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped))
	{
		if(Records.Num() > 0)
		{
			return Records[0].FindRef( TEXT("user") );
		}
	}
	else
	{
		for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
		{
			FMessageLog("SourceControl").Error(FText::Format(LOCTEXT("GetUserFromChangelistErrorFormat", "GetUserFromChangelist Error: {0}"), ErrorMsg));
		}
	}

	return FString();
}

/** 
 * Parse the results of a 'p4 annotate' command, outputting an array of lines.
 * @param	Records			The records to parse
 * @param	OutLines		An array that is filled with information about each line
 * @param	InConnection	The Perforce connection to use
 */
static void ParseAnnotationResults(const FP4RecordSet& Records, TArray<FAnnotationLine>& OutLines, FPerforceConnection& InConnection)
{
	TMap<int, FString> Users;
	for( int RecordIndex = 0; RecordIndex < Records.Num(); RecordIndex++ )
	{
		TMap<FString, FString> Tags = Records[RecordIndex];
			
		FString ChangeNumberString = Tags.FindRef( TEXT( "lower" ) );
		if(ChangeNumberString.Len() > 0)
		{
			FString Line = Tags.FindRef( TEXT( "data" ) ).Replace( TEXT( "\r" ), TEXT( "" ) ).Replace( TEXT( "\n" ), TEXT( "" ) );
			// we need to add the username to the annotation too
			// If we don't have the user for this change cached, look it up
			int32 ChangeNumber = FCString::Atoi(*ChangeNumberString);
			FString* FindUser = Users.Find( ChangeNumber );
			if( FindUser == NULL )
			{
				FString User = GetUserFromChangelist( ChangeNumber, InConnection );
				Users.Add( ChangeNumber, User );
				OutLines.Add(FAnnotationLine(ChangeNumber, User, Line));
			}
			else
			{
				OutLines.Add(FAnnotationLine(ChangeNumber, *FindUser, Line));
			}
		}
	}
}

bool FPerforceSourceControlRevision::GetAnnotated( TArray<FAnnotationLine>& OutLines ) const
{
	bool bCommandOK = false;
	
	FScopedPerforceConnection ScopedConnection(EConcurrency::Synchronous, GetSCCProvider());
	if(ScopedConnection.IsValid())
	{
		FPerforceConnection& Connection = ScopedConnection.GetConnection();
		FP4RecordSet Records;
		bool bConnectionDropped;
		FSourceControlResultInfo ResultInfo;
		TArray<FString> Parameters;

		Parameters.Add(TEXT("-q"));	// Suppress the one-line file header normally added by Perforce.
		Parameters.Add(TEXT("-c"));	// Display change numbers rather than revision numbers
		Parameters.Add(TEXT("-I"));	// Follow integrations

		const FString RevString = (RevisionNumber < 0) ? TEXT("head") : FString::Printf(TEXT("%d"), RevisionNumber);
		Parameters.Add(FileName + TEXT("#") + RevString);

		bCommandOK = Connection.RunCommand(TEXT("annotate"), Parameters, Records, ResultInfo, FOnIsCancelled(), bConnectionDropped);
		if(bCommandOK)
		{
			ParseAnnotationResults(Records, OutLines, Connection);
		}
		else
		{
			for (const FText& ErrorMsg : ResultInfo.ErrorMessages)
			{
				FMessageLog("SourceControl").Error(FText::Format(LOCTEXT("GetAnnotatedErrorFormat", "GetAnnotated Error: {0}"), ErrorMsg));
			}
		}
	}
	return bCommandOK;
}

bool FPerforceSourceControlRevision::GetAnnotated( FString& InOutFilename ) const
{
	TArray<FAnnotationLine> Lines;
	if(GetAnnotated(Lines))
	{
		FString FileBuffer;
		for(auto Iter(Lines.CreateConstIterator()); Iter; Iter++)
		{
			FileBuffer += FString::Printf( TEXT( "%8d %20s:\t%s\r\n" ), Iter->ChangeNumber, *Iter->UserName, *Iter->Line );
		}

		// Make temp filename to output to, or used a pass in one if there is one
		FString AbsoluteFileName;
		if(InOutFilename.Len() > 0)
		{
			AbsoluteFileName = InOutFilename;
		}
		else
		{
			const FString RevString = (RevisionNumber < 0) ? TEXT("head") : FString::Printf(TEXT("%d"), RevisionNumber);
			const FString AnnotatedName = FString::Printf(TEXT("Annotated-%s-Rev-%s-"), *FPaths::GetBaseFilename(FileName), *RevString);
			const FString Extension = TEXT(".") + FPaths::GetExtension(FileName);
			const FString TempFileName = FPaths::CreateTempFilename(*FPaths::DiffDir(), *AnnotatedName, *Extension);
			AbsoluteFileName = FPaths::ConvertRelativePathToFull(TempFileName);
		}

		if(!FFileHelper::SaveStringToFile(FileBuffer, *AbsoluteFileName, FFileHelper::EEncodingOptions::ForceAnsi))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add( TEXT("Filename"), FText::FromString(AbsoluteFileName) );
			FMessageLog("SourceControl").Error(FText::Format(LOCTEXT("FailedToWrite", "Failed to write to file: {Filename}"), Arguments));
			return false;
		}

		InOutFilename = AbsoluteFileName;
		return true;
	}

	return false;
}

const FString& FPerforceSourceControlRevision::GetFilename() const
{
	return FileName;
}

int32 FPerforceSourceControlRevision::GetRevisionNumber() const
{
	return RevisionNumber;
}

const FString& FPerforceSourceControlRevision::GetRevision() const
{
	return Revision;
}

const FString& FPerforceSourceControlRevision::GetDescription() const
{
	return Description;
}

const FString& FPerforceSourceControlRevision::GetUserName() const
{
	return UserName;
}

const FString& FPerforceSourceControlRevision::GetClientSpec() const
{
	return ClientSpec;
}

const FString& FPerforceSourceControlRevision::GetAction() const
{
	return Action;
}

TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> FPerforceSourceControlRevision::GetBranchSource() const
{
	return BranchSource;
}

const FDateTime& FPerforceSourceControlRevision::GetDate() const
{
	return Date;
}

int32 FPerforceSourceControlRevision::GetCheckInIdentifier() const
{
	return ChangelistNumber;
}

int32 FPerforceSourceControlRevision::GetFileSize() const
{
	return FileSize;
}

#undef LOCTEXT_NAMESPACE
