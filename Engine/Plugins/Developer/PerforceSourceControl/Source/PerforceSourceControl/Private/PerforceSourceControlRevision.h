// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"

class FPerforceConnection;
class FPerforceSourceControlProvider;

class FPerforceSourceControlRevision final : public ISourceControlRevision
{
public:

	FPerforceSourceControlRevision(FPerforceSourceControlProvider& InSCCProvider);
	virtual ~FPerforceSourceControlRevision() = default;

	bool Get(FString& InOutFilename, FPerforceConnection& InConnection) const;

	/** ISourceControlRevision interface */
	virtual bool Get( FString& InOutFilename, EConcurrency::Type InConcurrency = EConcurrency::Synchronous) const override;
	virtual bool GetAnnotated( TArray<FAnnotationLine>& OutLines ) const override;
	virtual bool GetAnnotated( FString& InOutFilename ) const override;
	virtual const FString& GetFilename() const override;
	virtual int32 GetRevisionNumber() const override;
	virtual const FString& GetRevision() const override;
	virtual const FString& GetDescription() const override;
	virtual const FString& GetUserName() const override;
	virtual const FString& GetClientSpec() const override;
	virtual const FString& GetAction() const override;
	virtual TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> GetBranchSource() const override;
	virtual const FDateTime& GetDate() const override;
	virtual int32 GetCheckInIdentifier() const override;
	virtual int32 GetFileSize() const override;

public:

	/** The local filename the this revision refers to */
	FString FileName;

	/** The revision number of this file */
	int32 RevisionNumber;

	/** The revision to display to the user */
	FString Revision;

	/** The changelist description of this revision */
	FString Description;

	/** THe user that made the change for this revision */
	FString UserName;

	/** The workspace the change was made from */
	FString ClientSpec;

	/** The action (edit, add etc.) that was performed for at this revision */
	FString Action;

	/** Source of branch, if any */
	TSharedPtr<FPerforceSourceControlRevision, ESPMode::ThreadSafe> BranchSource;

	/** The date of this revision */
	FDateTime Date;

	/** The changelist number of this revision */
	int32 ChangelistNumber;

	/** The size of the change */
	int32 FileSize;

	/** Whether this reprensents a revision bound to a shelved file in a changelist */
	bool bIsShelve;

private:
	/** Internal accessor to the source control provider associated with the object */
	FPerforceSourceControlProvider& GetSCCProvider() const
	{
		return SCCProvider;
	}

	/** The source control provider that this object is associated with */
	FPerforceSourceControlProvider& SCCProvider;
};
