// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlSettings.h"

#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "PerforceSourceControlPrivate.h"
#include "PerforceSourceControlProvider.h"
#include "SPerforceSourceControlSettings.h"
#include "SourceControlHelpers.h"

FPerforceSourceControlSettings::FPerforceSourceControlSettings(const FPerforceSourceControlProvider& InSCCProvider, const FStringView& OwnerName)
	: SCCProvider(InSCCProvider)
	, bCanSaveToIniFile(true)
	, bCanLoadFromIniFile(true)
{
	if (OwnerName.IsEmpty())
	{
		SettingsSection = TEXT("PerforceSourceControl.PerforceSourceControlSettings");
	}
	else
	{
		SettingsSection = WriteToString<128>(TEXT("PerforceSourceControl."), OwnerName, TEXT("Settings"));
	}
}

const FString& FPerforceSourceControlSettings::GetPort() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.Port;
}

void FPerforceSourceControlSettings::SetPort(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.Port = InString;
}

const FString& FPerforceSourceControlSettings::GetUserName() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.UserName;
}

void FPerforceSourceControlSettings::SetUserName(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.UserName = InString;
}

const FString& FPerforceSourceControlSettings::GetWorkspace() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.Workspace;
}

void FPerforceSourceControlSettings::SetWorkspace(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.Workspace = InString;
}

const FString& FPerforceSourceControlSettings::GetHostOverride() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.HostOverride;
}

void FPerforceSourceControlSettings::SetHostOverride(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.HostOverride = InString;
}

const FString& FPerforceSourceControlSettings::GetChangelistNumber() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.ChangelistNumber;
}

void FPerforceSourceControlSettings::SetChangelistNumber(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.ChangelistNumber = InString;
}

bool FPerforceSourceControlSettings::GetUseP4Config() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.bUseP4Config;
}

void FPerforceSourceControlSettings::SetUseP4Config(bool bInUseP4Config)
{
	FScopeLock ScopeLock(&CriticalSection);
	ConnectionInfo.bUseP4Config = bInUseP4Config;
	if (bInUseP4Config)
	{
		ImportP4Config();
	}
}

void FPerforceSourceControlSettings::ImportP4Config()
{
	ClientApi TestP4;
	ConnectionInfo.Port = ANSI_TO_TCHAR(TestP4.GetPort().Text());
	ConnectionInfo.UserName = ANSI_TO_TCHAR(TestP4.GetUser().Text());
	ConnectionInfo.Workspace = ANSI_TO_TCHAR(TestP4.GetClient().Text());
}

void FPerforceSourceControlSettings::SetAllowSave(bool bFlag)
{
	bCanSaveToIniFile = bFlag;
}

void FPerforceSourceControlSettings::SetAllowLoad(bool bFlag)
{
	bCanLoadFromIniFile = bFlag;
}

void FPerforceSourceControlSettings::LoadSettings()
{
	if (!bCanLoadFromIniFile)
	{
		return;
	}

	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();

	GConfig->GetBool(*SettingsSection, TEXT("UseP4Config"), ConnectionInfo.bUseP4Config, IniFile);
	if (ConnectionInfo.bUseP4Config)
	{
		ImportP4Config();
	}
	else
	{
		if(!GConfig->GetString(*SettingsSection, TEXT("Port"), ConnectionInfo.Port, IniFile))
		{
			// backwards compatibility - previously we mis-specified the Port as 'Host'
			GConfig->GetString(*SettingsSection, TEXT("Host"), ConnectionInfo.Port, IniFile);
		}
		GConfig->GetString(*SettingsSection, TEXT("UserName"), ConnectionInfo.UserName, IniFile);
		GConfig->GetString(*SettingsSection, TEXT("Workspace"), ConnectionInfo.Workspace, IniFile);
	}
	GConfig->GetString(*SettingsSection, TEXT("HostOverride"), ConnectionInfo.HostOverride, IniFile);
}

void FPerforceSourceControlSettings::SaveSettings() const
{
	if (FApp::IsUnattended() || IsRunningCommandlet() || !bCanSaveToIniFile)
	{
		return;
	}

	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	
	GConfig->SetBool(*SettingsSection, TEXT("UseP4Config"), ConnectionInfo.bUseP4Config, IniFile);
	
	auto AddOrRemoveKey = [this, &IniFile](const TCHAR* Key, const FString& Value)
		{
			if (!Value.IsEmpty())
			{
				GConfig->SetString(*SettingsSection, Key, *Value, IniFile);
			}
			else
			{
				GConfig->RemoveKey(*SettingsSection, Key, IniFile);
			}
		};

	AddOrRemoveKey(TEXT("Port"), ConnectionInfo.Port);
	AddOrRemoveKey(TEXT("UserName"), ConnectionInfo.UserName);
	AddOrRemoveKey(TEXT("Workspace"), ConnectionInfo.Workspace);
	AddOrRemoveKey(TEXT("HostOverride"), ConnectionInfo.HostOverride);
}

FPerforceConnectionInfo FPerforceSourceControlSettings::GetConnectionInfo() const
{
	FPerforceConnectionInfo OutConnectionInfo;
	{
		FScopeLock ScopeLock(&CriticalSection);
		OutConnectionInfo = ConnectionInfo;
	}
	
#if SOURCE_CONTROL_WITH_SLATE
	// The password needs to be gotten straight from the input UI, its not stored anywhere else
	const FString Password = SPerforceSourceControlSettings::GetPassword();
#else
	const FString Password;
#endif //SOURCE_CONTROL_WITH_SLATE

	if(!Password.IsEmpty())
	{
		OutConnectionInfo.Password = Password;
	}

	// Ticket is stored in the provider (this is only set by the command line so should be safe to access without threading protection)
	OutConnectionInfo.Ticket = GetSCCProvider().GetTicket();

	return OutConnectionInfo;
}
