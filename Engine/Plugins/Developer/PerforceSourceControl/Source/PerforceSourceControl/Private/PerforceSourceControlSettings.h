// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PerforceConnectionInfo.h"

class FPerforceSourceControlProvider;

class FPerforceSourceControlSettings
{
public:
	FPerforceSourceControlSettings(const FPerforceSourceControlProvider& InSCCProvider, const FStringView& OwnerName);
	~FPerforceSourceControlSettings() = default;

	/** Get the Perforce port */
	const FString& GetPort() const;

	/** Set the Perforce port */
	void SetPort(const FString& InString);

	/** Get the Perforce username */
	const FString& GetUserName() const;

	/** Set the Perforce username */
	void SetUserName(const FString& InString);

	/** Get the Perforce workspace */
	const FString& GetWorkspace() const;

	/** Set the Perforce workspace */
	void SetWorkspace(const FString& InString);

	/** Get the Perforce host override */
	const FString& GetHostOverride() const;

	/** Set the Perforce host override */
	void SetHostOverride(const FString& InString);

	/** Get the perforce cl we should use for this run (useful in commandlets) returns empty string if there is no cl*/
	const FString& GetChangelistNumber() const;

	void SetChangelistNumber(const FString& InString);

	/* Import the P4USER, P4PORT, P4CLIENT from the P4 environment if set. */
	bool GetUseP4Config() const;

	void SetUseP4Config(bool bInUseP4Config);

	/** When set to true the settings can be saved to an ini file, when false nothing will be saved */
	void SetAllowSave(bool bFlag);
	
	/** When set to true the settings can be read from an ini file, when false nothing will be read */
	void SetAllowLoad(bool bFlag);

	/** Load settings from ini file */
	void LoadSettings();

	/** Save settings to ini file */
	void SaveSettings() const;

	/** Get the credentials we use to access the server - only call on the game thread */
	FPerforceConnectionInfo GetConnectionInfo() const;

private:

	void ImportP4Config();
	
	/** A critical section for settings access */
	mutable FCriticalSection CriticalSection;

	/** The credentials we use to access the server */
	FPerforceConnectionInfo ConnectionInfo;
	
	/** The section of the ini file we load our settings from */
	FString SettingsSection;

	/** Internal accessor to the source control provider associated with the object */
	const FPerforceSourceControlProvider& GetSCCProvider() const
	{
		return SCCProvider;
	}

	/** The source control provider that this object is associated with */
	const FPerforceSourceControlProvider& SCCProvider;

	/** Controls whether the settings can be saved to an ini file or not */
	bool bCanSaveToIniFile;

	/** Controls whether the settings can be read from an ini file or not */
	bool bCanLoadFromIniFile;
};
