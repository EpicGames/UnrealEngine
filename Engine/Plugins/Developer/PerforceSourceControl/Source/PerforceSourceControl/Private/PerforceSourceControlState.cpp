// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerforceSourceControlState.h"
#include "PerforceSourceControlRevision.h"
#include "Misc/EngineVersion.h"
#include "HAL/IConsoleManager.h"

#if SOURCE_CONTROL_WITH_SLATE
#include "Textures/SlateIcon.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#endif //SOURCE_CONTROL_WITH_SLATE

#define LOCTEXT_NAMESPACE "PerforceSourceControl.State"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPerforceSourceControlState::FPerforceSourceControlState() = default;
FPerforceSourceControlState::FPerforceSourceControlState(const FPerforceSourceControlState& Other) = default;
FPerforceSourceControlState::FPerforceSourceControlState(FPerforceSourceControlState&& Other) noexcept = default;
FPerforceSourceControlState& FPerforceSourceControlState::operator=(const FPerforceSourceControlState& Other) = default;
FPerforceSourceControlState& FPerforceSourceControlState::operator=(FPerforceSourceControlState&& Other) noexcept = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace UE::RevisionControl::Perforce
{
	static bool bAllowRevertDeletedFiles = false;
	
	static FAutoConsoleVariableRef CVarAllowRevertDeletedFiles(
		TEXT("RevisionControl.Perforce.AllowRevertingDeletedFiles"),
		bAllowRevertDeletedFiles,
		TEXT("If true, deleted files can be reverted in the editor"),
		ECVF_Default
	);
}



int32 FPerforceSourceControlState::GetHistorySize() const
{
	return History.Num();
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPerforceSourceControlState::GetHistoryItem( int32 HistoryIndex ) const
{
	check(History.IsValidIndex(HistoryIndex));
	return History[HistoryIndex];
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPerforceSourceControlState::FindHistoryRevision( int32 RevisionNumber ) const
{
	for(auto Iter(History.CreateConstIterator()); Iter; Iter++)
	{
		if((*Iter)->GetRevisionNumber() == RevisionNumber)
		{
			return *Iter;
		}
	}

	return nullptr;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPerforceSourceControlState::FindHistoryRevision(const FString& InRevision) const
{
	for(const auto& Revision : History)
	{
		if(Revision->GetRevision() == InRevision)
		{
			return Revision;
		}
	}

	return nullptr;
}


bool FPerforceSourceControlState::IsCheckedOutInOtherBranch(const FString& CurrentBranch) const
{
	return (CheckedOutBranches.Num() > 0 && !CheckedOutBranches.Contains(CurrentBranch.Len() ? CurrentBranch : FEngineVersion::Current().GetBranch()));
}

bool FPerforceSourceControlState::IsModifiedInOtherBranch(const FString& CurrentBranch) const
{
	return !HeadBranch.IsEmpty() && (HeadBranch != TEXT("*CurrentBranch")) && ( HeadBranch != (CurrentBranch.Len() ? CurrentBranch : FEngineVersion::Current().GetBranch()));
}

bool FPerforceSourceControlState::GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const
{
	if (HeadBranch == TEXT("*CurrentBranch"))
	{
		return false;
	}

	HeadBranchOut = HeadBranch;
	ActionOut = HeadAction;
	HeadChangeListOut = HeadChangeList;
	
	return !HeadBranchOut.IsEmpty();
}

FSourceControlChangelistPtr FPerforceSourceControlState::GetCheckInIdentifier() const
{
	return MakeShared<FPerforceSourceControlChangelist>(Changelist);
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPerforceSourceControlState::GetCurrentRevision() const
{
	if (LocalRevNumber == INVALID_REVISION)
	{
		return nullptr;
	}

	return FindHistoryRevision(LocalRevNumber);
}

ISourceControlState::FResolveInfo FPerforceSourceControlState::GetResolveInfo() const
{
	return PendingResolveInfo;
}

#if SOURCE_CONTROL_WITH_SLATE
FSlateIcon FPerforceSourceControlState::GetIcon() const
{
	if (IsConflicted())
	{
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Conflicted");
	}
	if (!IsCurrent())
	{
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.NotAtHeadRevision");
	}
	else if (State != EPerforceState::CheckedOut && State != EPerforceState::CheckedOutOther)
	{
		if (IsCheckedOutInOtherBranch())
		{
			return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOutByOtherUserOtherBranch", NAME_None, "RevisionControl.CheckedOutByOtherUserOtherBranchBadge");
		}

		if (IsModifiedInOtherBranch())
		{
			return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ModifiedOtherBranch", NAME_None, "RevisionControl.ModifiedBadge");
		}
	}

	switch (State)
	{
	default:
	case EPerforceState::DontCare:
		return FSlateIcon();
	case EPerforceState::CheckedOut:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOut");
	case EPerforceState::ReadOnly:
		return FSlateIcon();
	case EPerforceState::NotInDepot:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.NotInDepot");
	case EPerforceState::CheckedOutOther:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOutByOtherUser", NAME_None, "RevisionControl.CheckedOutByOtherUserBadge");
	case EPerforceState::Ignore:
		return FSlateIcon();
	case EPerforceState::OpenForAdd:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.OpenForAdd");
	case EPerforceState::MarkedForDelete:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.MarkedForDelete");
	case EPerforceState::Branched:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Branched");
	}
}
#endif //SOURCE_CONTROL_WITH_SLATE

FText FPerforceSourceControlState::GetDisplayName() const
{
	if( IsConflicted() )
	{
		return LOCTEXT("Conflicted", "Conflicted");
	}
	else if( !IsCurrent() )
	{
		return LOCTEXT("NotCurrent", "Not current");
	}
	else if (State != EPerforceState::CheckedOut && State != EPerforceState::CheckedOutOther)
	{
		if (IsCheckedOutInOtherBranch())
		{
			return FText::Format(LOCTEXT("CheckedOutOther", "Checked out by: {0}"), FText::FromString(OtherUserBranchCheckedOuts));
		}

		if (IsModifiedInOtherBranch())
		{
			return FText::Format(LOCTEXT("ModifiedOtherBranch", "Modified in branch: {0}"), FText::FromString(HeadBranch));
		}
	}

	switch(State)
	{
	default:
	case EPerforceState::DontCare:
		return LOCTEXT("Unknown", "Unknown");
	case EPerforceState::CheckedOut:
		return LOCTEXT("CheckedOut", "Checked out");
	case EPerforceState::ReadOnly:
		return LOCTEXT("ReadOnly", "Read only");
	case EPerforceState::NotInDepot:
		return LOCTEXT("NotInDepot", "Not in depot");
	case EPerforceState::CheckedOutOther:
		return FText::Format( LOCTEXT("CheckedOutOther", "Checked out by: {0}"), FText::FromString(OtherUserCheckedOut) );
	case EPerforceState::Ignore:
		return LOCTEXT("Ignore", "Ignore");
	case EPerforceState::OpenForAdd:
		return LOCTEXT("OpenedForAdd", "Opened for add");
	case EPerforceState::MarkedForDelete:
		return LOCTEXT("MarkedForDelete", "Marked for delete");
	case EPerforceState::Branched:
		return LOCTEXT("Branched", "Branched");
	}
}

FText FPerforceSourceControlState::GetDisplayTooltip() const
{
	if (IsConflicted())
	{
		return LOCTEXT("Conflicted_Tooltip", "The file has local changes that need to be resolved with changes submitted to the Perforce depot");
	}
	else if( !IsCurrent() )
	{
		return LOCTEXT("NotCurrent_Tooltip", "The file is not at the head revision");
	}
	else if (State != EPerforceState::CheckedOut && State != EPerforceState::CheckedOutOther)
	{
		if (IsCheckedOutInOtherBranch())
		{
			return FText::Format(LOCTEXT("CheckedOutOther_Tooltip", "Checked out by: {0}"), FText::FromString(GetOtherUserBranchCheckedOuts()));
		}
		else if (IsModifiedInOtherBranch())
		{
			FNumberFormattingOptions NoCommas;
			NoCommas.UseGrouping = false;
			return FText::Format(LOCTEXT("ModifiedOtherBranch_Tooltip", "Modified in branch: {0} CL:{1} ({2})"), FText::FromString(HeadBranch), FText::AsNumber(HeadChangeList, &NoCommas), FText::FromString(HeadAction));
		}
	}

	switch(State)
	{
	default:
	case EPerforceState::DontCare:
		return LOCTEXT("Unknown_Tooltip", "The file status is unknown");
	case EPerforceState::CheckedOut:
		return LOCTEXT("CheckedOut_Tooltip", "The file is checked out");
	case EPerforceState::ReadOnly:
		return LOCTEXT("ReadOnly_Tooltip", "The file is marked locally as read-only");
	case EPerforceState::NotInDepot:
		return LOCTEXT("NotInDepot_Tooltip", "The file is not present in the Perforce depot");
	case EPerforceState::CheckedOutOther:
		return FText::Format( LOCTEXT("CheckedOutOther_Tooltip", "Checked out by: {0}"), FText::FromString(OtherUserCheckedOut) );
	case EPerforceState::Ignore:
		return LOCTEXT("Ignore_Tooltip", "The file is ignored by Perforce");
	case EPerforceState::OpenForAdd:
		return LOCTEXT("OpenedForAdd_Tooltip", "The file is opened for add");
	case EPerforceState::MarkedForDelete:
		return LOCTEXT("MarkedForDelete_Tooltip", "The file is marked for delete");
	case EPerforceState::Branched:
		return LOCTEXT("Branched_Tooltip", "The file is opened for branching");
	}
}

const FString& FPerforceSourceControlState::GetFilename() const
{
	return LocalFilename;
}

const FDateTime& FPerforceSourceControlState::GetTimeStamp() const
{
	return TimeStamp;
}

bool FPerforceSourceControlState::CanCheckIn() const
{
	return ( (State == EPerforceState::CheckedOut) 
			|| (State == EPerforceState::OpenForAdd) 
			|| (State == EPerforceState::MarkedForDelete)
			|| (State == EPerforceState::Branched) )
		&& !IsConflicted() && IsCurrent();
}

bool FPerforceSourceControlState::CanCheckout() const
{
	bool bCanDoCheckout = false;

	const bool bIsInP4NotCheckedOut = State == EPerforceState::ReadOnly;
	const bool bIsOpenForBranch = State == EPerforceState::Branched;
	if (!bBinary && !bExclusiveCheckout)
	{
		// Notice that we don't care whether we're up to date. User can perform textual
		// merge via P4V:
		const bool bIsCheckedOutElseWhere = State == EPerforceState::CheckedOutOther;
		bCanDoCheckout =	bIsInP4NotCheckedOut ||
							bIsCheckedOutElseWhere ||
							bIsOpenForBranch;
	}
	else
	{
		// For assets that are either binary or textual but marked for exclusive checkout
		// we only want to permit check out when we are at head:
		bCanDoCheckout = (bIsInP4NotCheckedOut || bIsOpenForBranch) && IsCurrent();
	}

	return bCanDoCheckout;
}

bool FPerforceSourceControlState::IsCheckedOut() const
{
	return State == EPerforceState::CheckedOut;
}

bool FPerforceSourceControlState::IsCheckedOutOther(FString* Who) const
{
	if(Who != nullptr)
	{
		*Who = OtherUserCheckedOut;
	}
	return State == EPerforceState::CheckedOutOther;
}

bool FPerforceSourceControlState::IsCurrent() const
{
	return LocalRevNumber == DepotRevNumber;
}

bool FPerforceSourceControlState::IsSourceControlled() const
{
	return State != EPerforceState::NotInDepot && State != EPerforceState::NotUnderClientRoot;
}

bool FPerforceSourceControlState::IsAdded() const
{
	return State == EPerforceState::OpenForAdd;
}

bool FPerforceSourceControlState::IsDeleted() const
{
	return State == EPerforceState::MarkedForDelete;
}

bool FPerforceSourceControlState::IsIgnored() const
{
	return State == EPerforceState::Ignore;
}

bool FPerforceSourceControlState::CanEdit() const
{
	return State == EPerforceState::CheckedOut || State == EPerforceState::OpenForAdd || State == EPerforceState::Branched;
}

bool FPerforceSourceControlState::CanDelete() const
{
	return !IsCheckedOutOther() && IsSourceControlled() && IsCurrent();
}

bool FPerforceSourceControlState::IsUnknown() const
{
	return State == EPerforceState::DontCare;
}

bool FPerforceSourceControlState::IsModified() const
{
	return bModifed;
}

bool FPerforceSourceControlState::CanAdd() const
{
	return State == EPerforceState::NotInDepot;
}

bool FPerforceSourceControlState::CanRevert() const
{
	bool bCanRevert = IsCheckedOut() || IsAdded();

	if (UE::RevisionControl::Perforce::CVarAllowRevertDeletedFiles->GetBool())
	{
		bCanRevert |= IsDeleted();
	}

	return bCanRevert;
}

void FPerforceSourceControlState::Update(const FPerforceSourceControlState& InOther, const FDateTime* InTimeStamp /* = nullptr */)
{
	check(InOther.LocalFilename == LocalFilename);

	if (InOther.History.Num() != 0)
	{
		History = InOther.History;
	}

	if (InOther.DepotFilename.Len() != 0)
	{
		DepotFilename = InOther.DepotFilename;
	}

	if (InOther.OtherUserCheckedOut.Len() != 0)
	{
		OtherUserCheckedOut = InOther.OtherUserCheckedOut;
	}

	if (InOther.State != EPerforceState::DontCare)
	{
		State = InOther.State;
	}

	if (InOther.DepotRevNumber != INVALID_REVISION)
	{
		DepotRevNumber = InOther.DepotRevNumber;
	}

	if (InOther.LocalRevNumber != INVALID_REVISION)
	{
		LocalRevNumber = InOther.LocalRevNumber;
	}

	if (InOther.PendingResolveInfo.IsValid())
	{
		PendingResolveInfo = InOther.PendingResolveInfo;
	}

	// This flag is true when the current object has been modified.
	// This is not necessarily known at all times, so we will play it safe and use an OR here.
	bModifed |= InOther.bModifed;

	// Here we will assume that the type info is always properly intiialized in InOther
	bBinary = InOther.bBinary;
	bExclusiveCheckout = InOther.bExclusiveCheckout;

	if (InOther.Changelist.IsInitialized())
	{
		Changelist = InOther.Changelist;
	}

	if (InTimeStamp)
	{
		TimeStamp = *InTimeStamp;
	}

	if (InOther.HeadBranch.Len() != 0)
	{
		HeadBranch = InOther.HeadBranch;
	}

	if (InOther.HeadAction.Len() != 0)
	{
		HeadAction = InOther.HeadAction;
	}

	if (InOther.HeadModTime != 0)
	{
		HeadModTime = InOther.HeadModTime;
	}

	if (InOther.HeadChangeList != 0)
	{
		HeadChangeList = InOther.HeadChangeList;
	}

	if (InOther.CheckedOutBranches.Num() != 0)
	{
		CheckedOutBranches = InOther.CheckedOutBranches;
	}

	if (InOther.OtherUserBranchCheckedOuts.Len() != 0)
	{
		OtherUserBranchCheckedOuts = InOther.OtherUserBranchCheckedOuts;
	}
}

#undef LOCTEXT_NAMESPACE
