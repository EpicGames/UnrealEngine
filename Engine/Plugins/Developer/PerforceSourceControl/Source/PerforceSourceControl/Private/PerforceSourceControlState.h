// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlState.h"
#include "ISourceControlRevision.h"

#include "PerforceSourceControlChangelist.h"

class FPerforceSourceControlRevision;

namespace EPerforceState
{
	enum Type
	{
		/** Don't know or don't care. */
		DontCare		= 0,

		/** File is checked out to current user. */
		CheckedOut		= 1,

		/** File is not checked out (but IS controlled by the source control system). */
		ReadOnly		= 2,

		/** File is new and not in the depot - needs to be added. */
		NotInDepot		= 4,

		/** File is checked out by another user and cannot be checked out locally. */
		CheckedOutOther	= 5,

		/** Certain packages are best ignored by the SCC system (MyLevel, Transient, etc). */
		Ignore			= 6,

		/** File is marked for add */
		OpenForAdd		= 7,

		/** File is marked for delete */
		MarkedForDelete	= 8,

		/** Not under client root or in client view */
		NotUnderClientRoot	= 9,

		/** Opened for branch */
		Branched = 10,
	};
}

class FPerforceSourceControlState : public ISourceControlState
{
public:
	FPerforceSourceControlState( const FString& InLocalFilename, EPerforceState::Type InState = EPerforceState::DontCare )
		: LocalFilename(InLocalFilename)
		, State(InState)
		, DepotRevNumber(INVALID_REVISION)
		, LocalRevNumber(INVALID_REVISION)
		, bModifed(false)
		, bBinary(false)
		, bExclusiveCheckout(false)
		, TimeStamp(0)
		, HeadModTime(0)
		, HeadChangeList(0)
	{
	}

	FPerforceSourceControlState();
	FPerforceSourceControlState(const FPerforceSourceControlState& Other);
	FPerforceSourceControlState(FPerforceSourceControlState&& Other) noexcept;
	FPerforceSourceControlState& operator=(const FPerforceSourceControlState& Other);
	FPerforceSourceControlState& operator=(FPerforceSourceControlState&& Other) noexcept;

	/** ISourceControlState interface */
	virtual int32 GetHistorySize() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetHistoryItem( int32 HistoryIndex ) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision( int32 RevisionNumber ) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision( const FString& InRevision ) const override;
	virtual FResolveInfo GetResolveInfo() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetCurrentRevision() const override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual FSlateIcon GetIcon() const override;
#endif //SOURCE_CONTROL_WITH_SLATE
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayTooltip() const override;
	virtual const FString& GetFilename() const override;
	virtual const FDateTime& GetTimeStamp() const override;
	virtual bool CanCheckIn() const override;
	virtual bool CanCheckout() const override;
	virtual bool IsCheckedOut() const override;
	virtual bool IsCheckedOutOther(FString* Who = nullptr) const override;
	virtual bool IsCheckedOutInOtherBranch(const FString& CurrentBranch = FString()) const override;
	virtual bool IsModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override;
	virtual bool IsCheckedOutOrModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override { return IsCheckedOutInOtherBranch(CurrentBranch) || IsModifiedInOtherBranch(CurrentBranch); }
	virtual TArray<FString> GetCheckedOutBranches() const override { return CheckedOutBranches; }
	virtual FString GetOtherUserBranchCheckedOuts() const override { return OtherUserBranchCheckedOuts; }
	virtual bool GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const override;
	virtual FSourceControlChangelistPtr GetCheckInIdentifier() const override;
	virtual bool IsCurrent() const override;
	virtual bool IsSourceControlled() const override;
	virtual bool IsAdded() const override;
	virtual bool IsDeleted() const override;
	virtual bool IsIgnored() const override;
	virtual bool CanEdit() const override;
	virtual bool IsUnknown() const override;
	virtual bool IsModified() const override;
	virtual bool CanAdd() const override;
	virtual bool CanDelete() const override;
	virtual bool CanRevert() const override;

	/** Get the state of a file */
	EPerforceState::Type GetState() const
	{
		return State;
	}

	/** Set the state of the file */
	void SetState( EPerforceState::Type InState )
	{
		State = InState;
	}

	/** Fills in missing information from other state */
	void Update(const FPerforceSourceControlState& InOther, const FDateTime* TimeStamp = nullptr);

public:
	/** History of the item, if any */
	TArray< TSharedRef<FPerforceSourceControlRevision, ESPMode::ThreadSafe> > History;

	/** Filename on disk */
	FString LocalFilename;

	/** Filename in the Perforce depot */
	FString DepotFilename;

	/** If another user has this file checked out, this contains their name(s). Multiple users are comma-delimited */
	FString OtherUserCheckedOut;

	/** Status of the file */
	EPerforceState::Type State;

	/** Latest revision number of the file in the depot */
	int DepotRevNumber;

	/** Latest rev number at which a file was synced to before being edited */
	int LocalRevNumber;

	/** Pending rev info with which a file must be resolved, invalid if no resolve pending */
	FResolveInfo PendingResolveInfo;

	UE_DEPRECATED(5.3, "Use PendingResolveInfo.BaseRevision instead")
	int PendingResolveRevNumber;

	/** Changelist containing this file */
	FPerforceSourceControlChangelist Changelist;

	/** Modified from depot version */
	bool bModifed;

	/** Whether the file is a binary file or not */
	bool bBinary;

	/** Whether the file is a marked for exclusive checkout or not */
	bool bExclusiveCheckout;

	/** The timestamp of the last update */
	FDateTime TimeStamp;

	/** The branch with the head change list */
	FString HeadBranch;

	/** The action within the head branch  */
	FString HeadAction;

	/** The last file modification time, note that P4 delete actions have a modification time of 0 */
	int64 HeadModTime;

	/** The change list the last modification */
	int32 HeadChangeList;

	/** Branches the file is checked out in */
	TArray<FString> CheckedOutBranches;

	/** Amalgamated information of users who have file checked out in another branch */
	FString OtherUserBranchCheckedOuts;

};
