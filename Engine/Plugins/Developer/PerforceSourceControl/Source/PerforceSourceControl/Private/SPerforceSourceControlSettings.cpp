// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPerforceSourceControlSettings.h"

#if SOURCE_CONTROL_WITH_SLATE

#include "ISourceControlModule.h"
#include "PerforceSourceControlInternalOperations.h"
#include "PerforceSourceControlPrivate.h"
#include "PerforceSourceControlProvider.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/AppStyle.h"
#include "ISourceControlModule.h"
#include "PerforceSourceControlModule.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

TWeakPtr<SEditableTextBox> SPerforceSourceControlSettings::PasswordTextBox;

#define LOCTEXT_NAMESPACE "SPerforceSourceControlSettings"

static bool bAllowP4NonTicketBasedLogins = false;
FAutoConsoleVariableRef CVarAllowP4NonTicketBasedLogins(
	TEXT("SourceControl.P4.AllowNonTicketLogins"),
	bAllowP4NonTicketBasedLogins,
	TEXT("Whether or not to allow logging in with a password directly from the perforce dialog. This is off by default because it is not a secure option. Perforce often your password as plain text in their enviroment variables")
);

void SPerforceSourceControlSettings::Construct(const FArguments& InArgs, FPerforceSourceControlProvider* InSCCProvider)
{
	checkf(InSCCProvider != nullptr, TEXT("SPerforceSourceControlSettings Requires a pointer to a valid FPerforceSourceControlProvider to function"));
	SCCProvider = InSCCProvider;

	bAreAdvancedSettingsExpanded = false;

	// check our settings & query if we don't already have any
	FString PortName = GetSCCProvider().AccessSettings().GetPort();
	FString UserName = GetSCCProvider().AccessSettings().GetUserName();

	if (PortName.IsEmpty() && UserName.IsEmpty())
	{
		ClientApi TestP4;
		TestP4.SetProg("UE");
		Error P4Error;
		TestP4.Init(&P4Error);
		PortName = ANSI_TO_TCHAR(TestP4.GetPort().Text());
		UserName = ANSI_TO_TCHAR(TestP4.GetUser().Text());
		TestP4.Final(&P4Error);
		
		GetSCCProvider().AccessSettings().SetPort(PortName);
		GetSCCProvider().AccessSettings().SetUserName(UserName);
		GetSCCProvider().AccessSettings().SaveSettings();
	}

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UseP4ConfigLabel", "Use P4 Config"))
				.ToolTipText( LOCTEXT("UseP4Config_Tooltip", "Read the P4USER, P4PORT, P4CLIENT from Perforce environment variables and P4CONFIG file rather than the UE project settings files") )
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PortLabel", "Server"))
				.ToolTipText( LOCTEXT("PortLabel_Tooltip", "The server and port for your Perforce server. Usage ServerName:1234.") )
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UserNameLabel", "User Name"))
				.ToolTipText( LOCTEXT("UserNameLabel_Tooltip", "Perforce username.") )
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WorkspaceLabel", "Workspace"))
				.ToolTipText( LOCTEXT("WorkspaceLabel_Tooltip", "Perforce workspace.") )
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AutoWorkspaces", "Available Workspaces"))
				.ToolTipText( LOCTEXT("AutoWorkspaces_Tooltip", "Choose from a list of available workspaces. Requires a server and username before use.") )
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Visibility(bAllowP4NonTicketBasedLogins ? EVisibility::Visible : EVisibility::Collapsed)
				.Text(LOCTEXT("HostLabel", "Host"))
				.ToolTipText(LOCTEXT("HostLabel_Tooltip", "If you wish to impersonate a particular host, enter this here. This is not normally needed."))
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Visibility(bAllowP4NonTicketBasedLogins ? EVisibility::Visible : EVisibility::Collapsed)
				.Text(LOCTEXT("PasswordLabel", "Password"))
				.ToolTipText(LOCTEXT("PasswordLabel_Tooltip", "Perforce password. This normally only needs to be entered if your ticket has expired."))
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(2.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			[
				SNew(SCheckBox)
				.IsChecked(this, &SPerforceSourceControlSettings::IsP4ConfigChecked)
				.OnCheckStateChanged(this, &SPerforceSourceControlSettings::OnP4ConfigCheckStatusChanged)
			]
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			[
				SNew(SEditableTextBox)
				.Text(this, &SPerforceSourceControlSettings::GetPortText)
				.ToolTipText( LOCTEXT("PortLabel_Tooltip", "The server and port for your Perforce server. Usage ServerName:1234.") )
				.OnTextCommitted(this, &SPerforceSourceControlSettings::OnPortTextCommitted)
				.OnTextChanged(this, &SPerforceSourceControlSettings::OnPortTextCommitted, ETextCommit::Default)
				.IsEnabled(this, &SPerforceSourceControlSettings::IsP4ConfigDisabled)
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPerforceSourceControlSettings::GetUserNameText)
				.ToolTipText( LOCTEXT("UserNameLabel_Tooltip", "Perforce username.") )
				.OnTextCommitted(this, &SPerforceSourceControlSettings::OnUserNameTextCommitted)
				.OnTextChanged(this, &SPerforceSourceControlSettings::OnUserNameTextCommitted, ETextCommit::Default)
				.IsEnabled(this, &SPerforceSourceControlSettings::IsP4ConfigDisabled)
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPerforceSourceControlSettings::GetWorkspaceText)
				.ToolTipText( LOCTEXT("WorkspaceLabel_Tooltip", "Perforce workspace.") )
				.OnTextCommitted(this, &SPerforceSourceControlSettings::OnWorkspaceTextCommitted)
				.OnTextChanged(this, &SPerforceSourceControlSettings::OnWorkspaceTextCommitted, ETextCommit::Default)
				.IsEnabled(this, &SPerforceSourceControlSettings::IsP4ConfigDisabled)
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(WorkspaceCombo, SComboButton)
				.OnGetMenuContent(this, &SPerforceSourceControlSettings::OnGetMenuContent)
				.ContentPadding(1.0f)
				.ToolTipText( LOCTEXT("AutoWorkspaces_Tooltip", "Choose from a list of available workspaces. Requires a server and username before use.") )
				.IsEnabled(this, &SPerforceSourceControlSettings::IsP4ConfigDisabled)
				.ButtonContent()
				[
					SNew( STextBlock )
					.Text( this, &SPerforceSourceControlSettings::OnGetButtonText )
				]
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Visibility(bAllowP4NonTicketBasedLogins ? EVisibility::Visible : EVisibility::Collapsed)
				.Text(this, &SPerforceSourceControlSettings::GetHostText)
				.ToolTipText(LOCTEXT("HostLabel_Tooltip", "If you wish to impersonate a particular host, enter this here. This is not normally needed."))
				.OnTextCommitted(this, &SPerforceSourceControlSettings::OnHostTextCommitted)
				.OnTextChanged(this, &SPerforceSourceControlSettings::OnHostTextCommitted, ETextCommit::Default)
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(PasswordTextBox, SEditableTextBox)
				.Visibility(bAllowP4NonTicketBasedLogins ? EVisibility::Visible : EVisibility::Collapsed)
				.ToolTipText(LOCTEXT("PasswordLabel_Tooltip", "Perforce password. This normally only needs to be entered if your ticket has expired."))
				.IsPassword(true)
			]
		]	
	];

	// fire off the workspace query
	State = ESourceControlOperationState::NotQueried;
	QueryWorkspaces();
}

FString SPerforceSourceControlSettings::GetPassword()
{
	if(PasswordTextBox.IsValid())
	{
		return PasswordTextBox.Pin()->GetText().ToString();
	}
	return FString();
}

void SPerforceSourceControlSettings::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// When a dialog is up, the editor stops ticking, and we take over:
	if( FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		ISourceControlModule::Get().Tick();
	}
	return SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime );
}

ECheckBoxState SPerforceSourceControlSettings::IsP4ConfigChecked() const
{
	return GetSCCProvider().AccessSettings().GetUseP4Config() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPerforceSourceControlSettings::OnP4ConfigCheckStatusChanged(ECheckBoxState NewState) const
{
	GetSCCProvider().AccessSettings().SetUseP4Config(NewState == ECheckBoxState::Checked);
	GetSCCProvider().AccessSettings().SaveSettings();
}

bool SPerforceSourceControlSettings::IsP4ConfigDisabled() const
{
	return !GetSCCProvider().AccessSettings().GetUseP4Config();
}

FText SPerforceSourceControlSettings::GetPortText() const
{
	return FText::FromString(GetSCCProvider().AccessSettings().GetPort());
}

void SPerforceSourceControlSettings::OnPortTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	GetSCCProvider().AccessSettings().SetPort(InText.ToString());
	GetSCCProvider().AccessSettings().SaveSettings();
}

FText SPerforceSourceControlSettings::GetUserNameText() const
{
	return FText::FromString(GetSCCProvider().AccessSettings().GetUserName());
}

void SPerforceSourceControlSettings::OnUserNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	GetSCCProvider().AccessSettings().SetUserName(InText.ToString());
	GetSCCProvider().AccessSettings().SaveSettings();
}

FText SPerforceSourceControlSettings::GetWorkspaceText() const
{
	return FText::FromString(GetSCCProvider().AccessSettings().GetWorkspace());
}

void SPerforceSourceControlSettings::OnWorkspaceTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	GetSCCProvider().AccessSettings().SetWorkspace(InText.ToString());
	GetSCCProvider().AccessSettings().SaveSettings();
}

FText SPerforceSourceControlSettings::GetHostText() const
{
	return FText::FromString(GetSCCProvider().AccessSettings().GetHostOverride());
}

void SPerforceSourceControlSettings::OnHostTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	GetSCCProvider().AccessSettings().SetHostOverride(InText.ToString());
	GetSCCProvider().AccessSettings().SaveSettings();
}

void SPerforceSourceControlSettings::QueryWorkspaces()
{
	if(State != ESourceControlOperationState::Querying)
	{
		Workspaces.Empty();
		CurrentWorkspace = FString();

		// fire off the workspace query
		GetWorkspacesOperation = ISourceControlOperation::Create<FGetProjectWorkspaces>();
		GetSCCProvider().Execute(GetWorkspacesOperation.ToSharedRef(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPerforceSourceControlSettings::OnSourceControlOperationComplete) );

		State = ESourceControlOperationState::Querying;
	}
}

void SPerforceSourceControlSettings::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	if(InResult == ECommandResult::Succeeded)
	{
		check(InOperation->GetName() == "GetProjectWorkspaces");
		check(GetWorkspacesOperation == StaticCastSharedRef<FGetProjectWorkspaces>(InOperation));

		// refresh workspaces list from operation results
		Workspaces.Empty();
		for(auto Iter(GetWorkspacesOperation->Results.CreateConstIterator()); Iter; Iter++)
		{
			Workspaces.Add(MakeShareable(new FString(*Iter)));
		}
	}

	GetWorkspacesOperation.Reset();
	State = ESourceControlOperationState::Queried;
}

TSharedRef<SWidget> SPerforceSourceControlSettings::OnGetMenuContent()
{
	// fire off the workspace query - we may have just edited the settings
	QueryWorkspaces();
	
	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPerforceSourceControlSettings::GetThrobberVisibility)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SThrobber)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WorkspacesOperationInProgress", "Looking for Perforce workspaces..."))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))	
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.OnClicked(this, &SPerforceSourceControlSettings::OnCancelWorkspacesRequest)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
				]
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoWorkspaces", "No Workspaces found!"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))	
			.Visibility(this, &SPerforceSourceControlSettings::GetNoWorkspacesVisibility)
		]
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SListView< TSharedRef<FString> >)
			.ListItemsSource(&Workspaces)
			.OnGenerateRow(this, &SPerforceSourceControlSettings::OnGenerateWorkspaceRow)
			.Visibility(this, &SPerforceSourceControlSettings::GetWorkspaceListVisibility)
			.OnSelectionChanged(this, &SPerforceSourceControlSettings::OnWorkspaceSelected)
		];
}

EVisibility SPerforceSourceControlSettings::GetThrobberVisibility() const
{
	return State == ESourceControlOperationState::Querying ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPerforceSourceControlSettings::GetNoWorkspacesVisibility() const
{
	return State == ESourceControlOperationState::Queried && Workspaces.Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPerforceSourceControlSettings::GetWorkspaceListVisibility() const
{
	return State == ESourceControlOperationState::Queried && Workspaces.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<ITableRow> SPerforceSourceControlSettings::OnGenerateWorkspaceRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return
		SNew(SComboRow< TSharedRef<FString> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(*InItem))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
}

void SPerforceSourceControlSettings::OnWorkspaceSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo)
{
	CurrentWorkspace = *InItem;
	
	GetSCCProvider().AccessSettings().SetWorkspace(CurrentWorkspace);
	GetSCCProvider().AccessSettings().SaveSettings();
	WorkspaceCombo->SetIsOpen(false);
}

FText SPerforceSourceControlSettings::OnGetButtonText() const
{
	return FText::FromString(CurrentWorkspace);
}

FReply SPerforceSourceControlSettings::OnCancelWorkspacesRequest() const
{
	if(GetWorkspacesOperation.IsValid())
	{
		ISourceControlModule& SourceControl = FModuleManager::LoadModuleChecked<ISourceControlModule>( "SourceControl" );
		GetSCCProvider().CancelOperation(GetWorkspacesOperation.ToSharedRef());
	}
	return FReply::Handled();
}

const FSlateBrush* SPerforceSourceControlSettings::GetAdvancedPulldownImage() const
{
	if( ExpanderButton->IsHovered() )
	{
		return bAreAdvancedSettingsExpanded ? FAppStyle::GetBrush("DetailsView.PulldownArrow.Up.Hovered") : FAppStyle::GetBrush("DetailsView.PulldownArrow.Down.Hovered");
	}
	else
	{
		return bAreAdvancedSettingsExpanded ? FAppStyle::GetBrush("DetailsView.PulldownArrow.Up") : FAppStyle::GetBrush("DetailsView.PulldownArrow.Down");
	}
}

EVisibility SPerforceSourceControlSettings::GetAdvancedSettingsVisibility() const
{
	return bAreAdvancedSettingsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SPerforceSourceControlSettings::OnAdvancedSettingsClicked()
{
	bAreAdvancedSettingsExpanded = !bAreAdvancedSettingsExpanded;
	return FReply::Handled();
}

FPerforceSourceControlProvider& SPerforceSourceControlSettings::GetSCCProvider() const
{
	check(SCCProvider != nullptr);
	return *SCCProvider;
}

#undef LOCTEXT_NAMESPACE

#endif //SOURCE_CONTROL_WITH_SLATE
