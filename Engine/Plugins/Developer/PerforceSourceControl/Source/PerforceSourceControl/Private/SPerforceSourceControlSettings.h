// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if SOURCE_CONTROL_WITH_SLATE

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "SourceControlOperationBase.h"
#include "ISourceControlProvider.h"

class FGetProjectWorkspaces;
class FPerforceSourceControlProvider;

class SButton;
class SComboButton;
class SEditableTextBox;

/** Enum for source control operation state */
namespace ESourceControlOperationState
{
	enum Type
	{
		NotQueried,
		Querying,
		Queried
	};
}

class SPerforceSourceControlSettings : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPerforceSourceControlSettings) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, FPerforceSourceControlProvider* InSCCProvider);

	/** Get the currently entered password */
	static FString GetPassword();

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime );
private:

	/** Fire off a source control operation to see what workspaces we have */
	void QueryWorkspaces();

	/** Delegate to check if we should use P4 Config from settings */
	ECheckBoxState IsP4ConfigChecked() const;

	/** Delegate to enable or disable P4 Config */
	void OnP4ConfigCheckStatusChanged(ECheckBoxState NewState) const;

	/** Delegate to enable Port, Username and input when P4 Config is not used */
	bool IsP4ConfigDisabled() const;

	/** Delegate to get port text from settings */
	FText GetPortText() const;

	/** Delegate to commit port text to settings */
	void OnPortTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const;

	/** Delegate to get user name text from settings */
	FText GetUserNameText() const;

	/** Delegate to commit user name text to settings */
	void OnUserNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const;

	/** Delegate to get workspace text from settings */
	FText GetWorkspaceText() const;

	/** Delegate to commit workspace text to settings */
	void OnWorkspaceTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const;

	/** Delegate to get host text from settings */
	FText GetHostText() const;

	/** Delegate to commit host text to settings */
	void OnHostTextCommitted(const FText& InText, ETextCommit::Type InCommitType) const;

	/** Delegate called when a source control operation has completed */
	void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Delegate for the auto workspaces dropdown menu content */
	TSharedRef<SWidget> OnGetMenuContent();

	/** Delegate controlling the visibility of the throbber */
	EVisibility GetThrobberVisibility() const;

	/** Delegate controlling the visibility 'no workspaces' message */
	EVisibility GetNoWorkspacesVisibility() const;

	/** Delegate controlling the visibility of the workspace list */
	EVisibility GetWorkspaceListVisibility() const;

	/** Delegate to generate a row in the workspace list */
	TSharedRef<class ITableRow> OnGenerateWorkspaceRow(TSharedRef<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Delegate for when a workspace item is clicked */
	void OnWorkspaceSelected(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectInfo);

	/** Delegate for the auto workspaces dropdown button text */
	FText OnGetButtonText() const;

	/** Delegate that cancels the workspace request in progress */
	FReply OnCancelWorkspacesRequest() const;

	/** Get the image to display on the advanced pulldown */
	const FSlateBrush* GetAdvancedPulldownImage() const;

	/** Get the visibility of the advanced settings */
	EVisibility GetAdvancedSettingsVisibility() const;

	/** Toggle advanced settings */
	FReply OnAdvancedSettingsClicked();

	FPerforceSourceControlProvider& GetSCCProvider() const;

private:
	FPerforceSourceControlProvider* SCCProvider;

	/** State of the source control operation */
	ESourceControlOperationState::Type State;

	/** The currently selected workspace */
	FString CurrentWorkspace;

	/** Workspaces we have received */
	TArray<TSharedRef<FString>> Workspaces;

	/** The combo button use to display the workspaces */
	TSharedPtr<SComboButton> WorkspaceCombo;

	/** The source control operation in progress */
	TSharedPtr<FGetProjectWorkspaces, ESPMode::ThreadSafe> GetWorkspacesOperation;

	/** Pointer to the password text box widget */
	static TWeakPtr<SEditableTextBox> PasswordTextBox;

	/** Expander button for advanced settings */
	TSharedPtr<SButton> ExpanderButton;

	/** Whether to expand advanced settings */
	bool bAreAdvancedSettingsExpanded;
};

#endif //SOURCE_CONTROL_WITH_SLATE
