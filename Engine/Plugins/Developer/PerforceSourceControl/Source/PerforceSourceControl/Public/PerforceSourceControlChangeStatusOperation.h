// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SourceControlOperationBase.h"

enum class EChangelistStatus
{
	Have,
	Need,
	Partial
};

struct FChangelistStatusEntry
{
	FString ChangelistNumber;
	EChangelistStatus Status;
};

/**
 * This operation wraps the p4 cstat command which returns a list of all changelists for a given
 * path and whether they are synced (have), will need to be synced (need) or partially synced.
 * As this is somewhat specific to Perforce it can only be used directly, it is not part of the shared operation
 * interfaces in SourceControlOperations.h
 */
class FPerforceSourceControlChangeStatusOperation : public FSourceControlOperationBase
{
public:
	virtual ~FPerforceSourceControlChangeStatusOperation() = default;

	virtual FName GetName() const override { return "ChangeStatus"; }

	/** Entries will be in the order they were reported in by perforce  */
	TArray<FChangelistStatusEntry> OutResults;
};
